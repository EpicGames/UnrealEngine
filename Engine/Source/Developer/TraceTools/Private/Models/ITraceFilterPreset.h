// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

namespace UE::TraceTools
{

class ITraceObject;

/** Interface for structures representing a filtering preset */
struct ITraceFilterPreset : public TSharedFromThis<ITraceFilterPreset>
{
public:
	virtual ~ITraceFilterPreset() = default;

	virtual FString GetName() const = 0;
	virtual FText GetDisplayText() const = 0;
	virtual FText GetDescription() const = 0;

	/** Returns all names which pass this preset its filter */
	virtual void GetAllowlistedNames(TArray<FString>& OutNames) const = 0;
	/** Whether or not this preset can be deleted */
	virtual bool CanDelete() const = 0;
	/** Whether or not this preset only exists locally */
	virtual bool IsLocal() const = 0;
	/** Whether or not this preset is a runtime preset */
	virtual bool IsEnginePreset() const = 0;

	/** Update this preset according to the provided set of trace objects */
	virtual void Save(const TArray<TSharedPtr<ITraceObject>>& InObjects) = 0;

	virtual void Save() = 0;

	virtual void Rename(const FString& InNewName) = 0;	
	virtual bool Delete() = 0;

	/** Transition preset between being local and shared, moves between local and default INI files */
	virtual bool MakeShared() = 0;
	virtual bool MakeLocal() = 0;
	
};

} // namespace UE::TraceTools
