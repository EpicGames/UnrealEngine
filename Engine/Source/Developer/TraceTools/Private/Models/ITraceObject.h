// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

namespace UE::TraceTools
{

class ITraceObject : public TSharedFromThis<ITraceObject>
{
public:
	virtual ~ITraceObject() = default;

	/** Returns the name representing this object */
	virtual FString GetName() const = 0;

	/** Returns the description representing this object */
	virtual FString GetDescription() const = 0;
	
	/** Returns the user-facing name for this object */
	virtual FText GetDisplayText() const = 0;

	/** Returns the tooltip text for this object */
	virtual FText GetTooltipText() const = 0;

	/** Set whether or not this object should (not) be filtered out */
	virtual void SetIsFiltered(bool bState) = 0;

	/** Returns whether or not this object can be changed by the user. */
	virtual bool IsReadOnly() const = 0;
	
	/** Set whether or not this object is pending an update of its state (from the connect edTrace connection) */
	virtual void SetPending() = 0;
	
	/** Returns whether or not this object is set to be filtered out (true = filtered; false = not filtered)*/
	virtual bool IsFiltered() const = 0;

	/** Returns whether or not this object is pending a filter-state update (SetPending previously called) */
	virtual bool IsPending() const = 0;

	/** Returns string representing this object in a search (treeview filtering) */
	virtual void GetSearchString(TArray<FString>& OutFilterStrings) const = 0;
};

} // namespace UE::TraceTools
