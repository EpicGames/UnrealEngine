// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceChannel.h"

#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"

// TraceTools
#include "Services/ISessionTraceFilterService.h"

#define LOCTEXT_NAMESPACE "UE::TraceTools::FTraceChannel"

namespace UE::TraceTools
{

FTraceChannel::FTraceChannel(FString InName,
							 FString InDescription,
							 FString InParentName, 
							 uint32 InId,
							 bool bInEnabled,
							 bool bInReadOnly,
							 TSharedPtr<ISessionTraceFilterService> InFilterService) 
	: Name(InName),
	  Description(InDescription),
	  ParentName(InParentName),
	  Id(InId),
	  bFiltered(!bInEnabled), 
	  bIsPending(false),
	  bReadOnly(bInReadOnly),
	  FilterService(InFilterService)
{
}

FText FTraceChannel::GetDisplayText() const
{
	return FText::FromString(Name);
}

FText FTraceChannel::GetTooltipText() const
{
	FText ChannelTooltip = FText::FromString(Description);
	if (bReadOnly)
	{
		if (Description.EndsWith("."))
		{
			ChannelTooltip = FText::Format(LOCTEXT("ChannelTooltipFmt1", "{0} This channel is readonly and can only be enabled from the command line."), FText::FromString(Description));
		}
		else
		{
			ChannelTooltip = FText::Format(LOCTEXT("ChannelTooltipFmt2", "{0}. This channel is readonly and can only be enabled from the command line."), FText::FromString(Description));
		}
	}

	return ChannelTooltip;
}

FString FTraceChannel::GetName() const
{
	return Name;
}

FString FTraceChannel::GetDescription() const
{
	return Description;
}

void FTraceChannel::SetPending()
{
	bIsPending = true;
}

bool FTraceChannel::IsReadOnly() const
{
	return bReadOnly;
}

void FTraceChannel::SetIsFiltered(bool bState)
{
	SetPending();
	FilterService->SetObjectFilterState(Name, !bState);
}

bool FTraceChannel::IsFiltered() const
{
	return bFiltered;
}

bool FTraceChannel::IsPending() const
{
	return bIsPending;
}

void FTraceChannel::GetSearchString(TArray<FString>& OutFilterStrings) const
{
	OutFilterStrings.Add(Name);
}

} // namespace UE::TraceTools

#undef LOCTEXT_NAMESPACE // UE::TraceTools::FTraceChannel
