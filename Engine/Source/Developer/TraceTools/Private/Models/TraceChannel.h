// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ITraceObject.h"

namespace UE::TraceTools
{

class ISessionTraceFilterService;

class FTraceChannel : public ITraceObject
{
public:
	FTraceChannel(FString InName, FString InDescription, FString InParentName, uint32 InId, bool bInEnabled, bool bInReadOnly, TSharedPtr<ISessionTraceFilterService> InFilterService);

	/** Begin ITraceObject overrides */
	virtual FText GetDisplayText() const override;
	virtual FText GetTooltipText() const override;
	virtual FString GetName() const override;
	virtual FString GetDescription() const override;
	virtual void SetPending() override;
	virtual bool IsReadOnly() const override;
	virtual void SetIsFiltered(bool bState) override;
	virtual bool IsFiltered() const override;
	virtual bool IsPending() const override;
	virtual void GetSearchString(TArray<FString>& OutFilterStrings) const override;
	/** End ITraceObject overrides */

protected:
	/** This channel's name */
	FString Name;
	/** This channel's description */
	FString Description;
	/** Channel's parent (group) name */
	FString ParentName;
	/** Channel's id */
	uint32 Id;
	
	/** Whether or not this channel is filtered out, true = filtered; false = not filtered */
	bool bFiltered;
	bool bIsPending;	
	bool bReadOnly;

	TSharedPtr<ISessionTraceFilterService> FilterService;
};

} // namespace UE::TraceTools