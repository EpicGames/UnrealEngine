// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/Commands.h"
#include "TraceToolsStyle.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "SessionConsoleCommands"

namespace UE::TraceTools
{

class FTraceControlCommands
	: public TCommands<FTraceControlCommands>
{
public:

	/** Default constructor. */
	FTraceControlCommands()
		: TCommands<FTraceControlCommands>(
			"TraceControl",
			NSLOCTEXT("Contexts", "TraceControl", "Trace Control"),
			NAME_None, FTraceToolsStyle::GetStyleSetName()
		)
	{ }

public:
	// TCommands interface
	virtual void RegisterCommands() override
	{
		UI_COMMAND(SetTraceTargetServer, "Trace Server", "Set the trace server as the trace target.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(SetTraceTargetFile, "File", "Set file as the trace target.", EUserInterfaceActionType::Button, FInputChord());

		UI_COMMAND(StartTrace, "Start Trace", "Start trace on the selected session.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(StopTrace, "Stop Trace", "Stop trace on the selected session.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(TraceSnapshot, "Snapshot", "Save a trace snapshot on the selected session.", EUserInterfaceActionType::Button, FInputChord());

		UI_COMMAND(TraceBookmark, "Bookmark", "Trace a bookmark with a predefined name.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(TraceScreenshot, "Screenshot", "Trace a screenshot with a predefined name.", EUserInterfaceActionType::Button, FInputChord());

		UI_COMMAND(ToggleStatNamedEvents, "Stat Named Events", "Significantly increases the number of CPU profiler events by activating tracing for the named events system.", EUserInterfaceActionType::ToggleButton, FInputChord());
	}

public:
	TSharedPtr<FUICommandInfo> SetTraceTargetServer;
	TSharedPtr<FUICommandInfo> SetTraceTargetFile;
	TSharedPtr<FUICommandInfo> StartTrace;
	TSharedPtr<FUICommandInfo> StopTrace;
	TSharedPtr<FUICommandInfo> TraceSnapshot;
	TSharedPtr<FUICommandInfo> TraceBookmark;
	TSharedPtr<FUICommandInfo> TraceScreenshot;
	TSharedPtr<FUICommandInfo> ToggleStatNamedEvents;
};

} // namespace UE::TraceTools

#undef LOCTEXT_NAMESPACE
