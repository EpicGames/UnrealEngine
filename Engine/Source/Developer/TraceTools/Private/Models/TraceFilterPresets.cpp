// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceFilterPresets.h"

#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/StringBuilder.h"

// TraceTools
#include "ITraceObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TraceFilterPresets)

#define LOCTEXT_NAMESPACE "TraceFilterPreset"

void USharedTraceFilterPresetContainer::GetSharedUserPresets(TArray<TSharedPtr<UE::TraceTools::ITraceFilterPreset>>& OutPresets)
{
	for (FTraceFilterData& FilterData : SharedPresets)
	{
		OutPresets.Add(MakeShared<UE::TraceTools::FUserFilterPreset>(FilterData.Name, FilterData));
	}
}

void USharedTraceFilterPresetContainer::AddFilterData(const FTraceFilterData& InFilterData)
{
	USharedTraceFilterPresetContainer* SharedPresetsContainer = GetMutableDefault<USharedTraceFilterPresetContainer>();
	SharedPresetsContainer->SharedPresets.Add(InFilterData);
}

bool USharedTraceFilterPresetContainer::RemoveFilterData(const FTraceFilterData& InFilterData)
{
	USharedTraceFilterPresetContainer* SharedPresetsContainer = GetMutableDefault<USharedTraceFilterPresetContainer>();
	return SharedPresetsContainer->SharedPresets.RemoveSingle(InFilterData) == 1;
}

void USharedTraceFilterPresetContainer::Save()
{
	USharedTraceFilterPresetContainer* SharedPresetsContainer = GetMutableDefault<USharedTraceFilterPresetContainer>();
	SharedPresetsContainer->TryUpdateDefaultConfigFile();
}

void ULocalTraceFilterPresetContainer::GetUserPresets(TArray<TSharedPtr<UE::TraceTools::ITraceFilterPreset>>& OutPresets)
{
	for (FTraceFilterData& FilterData : UserPresets)
	{
		const bool bIsLocal = true;
		OutPresets.Add(MakeShared<UE::TraceTools::FUserFilterPreset>(FilterData.Name, FilterData, bIsLocal));
	}
}

void ULocalTraceFilterPresetContainer::AddFilterData(const FTraceFilterData& InFilterData)
{
	ULocalTraceFilterPresetContainer* LocalPresetsContained = GetMutableDefault<ULocalTraceFilterPresetContainer>();
	LocalPresetsContained->UserPresets.Add(InFilterData);
}

bool ULocalTraceFilterPresetContainer::RemoveFilterData(const FTraceFilterData& InFilterData)
{
	ULocalTraceFilterPresetContainer* LocalPresetsContained = GetMutableDefault<ULocalTraceFilterPresetContainer>();
	return LocalPresetsContained->UserPresets.RemoveSingle(InFilterData) == 1;
}

void ULocalTraceFilterPresetContainer::Save()
{
	ULocalTraceFilterPresetContainer* LocalPresetsContainer = GetMutableDefault<ULocalTraceFilterPresetContainer>();
	LocalPresetsContainer->SaveConfig();
}

namespace UE::TraceTools
{

void FFilterPresetHelpers::CreateNewPreset(const TArray<TSharedPtr<ITraceObject>>& InObjects)
{
	ULocalTraceFilterPresetContainer* LocalPresetsContainer = GetMutableDefault<ULocalTraceFilterPresetContainer>();
	USharedTraceFilterPresetContainer* SharedPresetsContainer = GetMutableDefault<USharedTraceFilterPresetContainer>();

	FTraceFilterData& NewUserFilter = LocalPresetsContainer->UserPresets.AddDefaulted_GetRef();

	bool bFoundValidName = false;
	int32 ProfileAppendNum = LocalPresetsContainer->UserPresets.Num();
	FString NewFilterName;
	while (!bFoundValidName)
	{
		NewFilterName = FString::Printf(TEXT("UserPreset_%i"), ProfileAppendNum);

		bool bValidName = true;
		for (const FTraceFilterData& Filter : LocalPresetsContainer->UserPresets)
		{
			if (Filter.Name == NewFilterName)
			{
				bValidName = false;
				break;
			}
		}

		for (const FTraceFilterData& Filter : SharedPresetsContainer->SharedPresets)
		{
			if (Filter.Name == NewFilterName)
			{
				bValidName = false;
				break;
			}
		}

		if (!bValidName)
		{
			++ProfileAppendNum;
		}

		bFoundValidName = bValidName;
	}

	NewUserFilter.Name = NewFilterName;

	TArray<FString> Names;
	ExtractEnabledObjectNames(InObjects, Names);
	NewUserFilter.AllowlistedNames = Names;

	ULocalTraceFilterPresetContainer::Save();
}

bool FFilterPresetHelpers::CanModifySharedPreset()
{
	const USharedTraceFilterPresetContainer* SharedContainer = GetDefault<USharedTraceFilterPresetContainer>();
	return !FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(*SharedContainer->GetDefaultConfigFilename());
}

void FFilterPresetHelpers::ExtractEnabledObjectNames(const TArray<TSharedPtr<ITraceObject>>& InObjects, TArray<FString>& OutNames)
{
	for (const TSharedPtr<ITraceObject>& Object : InObjects)
	{
		if (!Object->IsFiltered())
		{
			OutNames.Add(Object->GetName());
		}
	}
}

bool FUserFilterPreset::CanDelete() const
{
	return true;
}

bool FUserFilterPreset::Delete()
{
	bool bRemoved = false;

	if (IsLocal())
	{
		bRemoved = ULocalTraceFilterPresetContainer::RemoveFilterData(FilterData);
		ULocalTraceFilterPresetContainer::Save();
	}
	else
	{
		bRemoved = USharedTraceFilterPresetContainer::RemoveFilterData(FilterData);
		USharedTraceFilterPresetContainer::Save();
	}

	return ensure(bRemoved);
}

bool FUserFilterPreset::MakeShared()
{
	ensure(IsLocal());

	USharedTraceFilterPresetContainer::AddFilterData(FilterData);
	ensure(ULocalTraceFilterPresetContainer::RemoveFilterData(FilterData));
		
	USharedTraceFilterPresetContainer::Save();
	ULocalTraceFilterPresetContainer::Save();
	
	return true;
}

bool FUserFilterPreset::MakeLocal()
{
	ensure(!IsLocal());

	ULocalTraceFilterPresetContainer::AddFilterData(FilterData);
	ensure(USharedTraceFilterPresetContainer::RemoveFilterData(FilterData));
	
	USharedTraceFilterPresetContainer::Save();
	ULocalTraceFilterPresetContainer::Save();

	return true;
}

bool FUserFilterPreset::IsLocal() const
{
	return bIsLocalPreset;
}

void FUserFilterPreset::Save(const TArray<TSharedPtr<ITraceObject>>& InObjects)
{
	TArray<FString> Names;
	FFilterPresetHelpers::ExtractEnabledObjectNames(InObjects, Names);

	FilterData.AllowlistedNames = Names;
	
	USharedTraceFilterPresetContainer::Save();
	ULocalTraceFilterPresetContainer::Save();
}

void FUserFilterPreset::Save()
{
	USharedTraceFilterPresetContainer::Save();
	ULocalTraceFilterPresetContainer::Save();
}

FString FFilterPreset::GetName() const
{
	return Name;
}

FText FFilterPreset::GetDisplayText() const
{
	return FText::FromString(Name);
}

FText FFilterPresetBase::GetDescription() const
{
	TStringBuilder<256> NamesList;
	TArray<FString> Names;
	GetAllowlistedNames(Names);

	for (const FString& NameEntry : Names)
	{
		NamesList.Append(NameEntry);
		NamesList.Append(TEXT(", "));
	}
	
	if (NamesList.Len() > 1)
	{
		NamesList.RemoveSuffix(2);
	}

	return FText::FormatOrdered(LOCTEXT("FilterPresetDescriptionFormat", "Name: {0}\nType: {1}\nAllowlist: {2}"), FText::FromString(Name), CanDelete() ? (IsLocal() ? LOCTEXT("LocalPreset", "Local") : LOCTEXT("SharedPreset", "Shared")) : LOCTEXT("EnginePreset", "Engine"), FText::FromString(NamesList.ToString()));
}

void FFilterPreset::GetAllowlistedNames(TArray<FString>& OutNames) const
{
	OutNames.Append(FilterData.AllowlistedNames);
}

bool FFilterPreset::CanDelete() const
{
	return false;
}

void FFilterPreset::Rename(const FString& InNewName)
{	
	Name = InNewName;
	FilterData.Name = InNewName;

	Save();
}

bool FFilterPreset::Delete()
{
	return false;
}

bool FFilterPreset::MakeShared()
{
	return false;
}

bool FFilterPreset::MakeLocal()
{
	return false;
}

bool FFilterPreset::IsLocal() const
{
	return false;
}

void FEngineFilterPreset::GetAllowlistedNames(TArray<FString>& OutNames) const
{
	OutNames.Insert(AllowListedNames, 0);
}

} // namespace UE::TraceTools

#undef LOCTEXT_NAMESPACE // "FilterPreset"

