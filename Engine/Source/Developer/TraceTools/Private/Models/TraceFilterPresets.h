// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ITraceFilterPreset.h"

#include "TraceFilterPresets.generated.h"

namespace UE::TraceTools
{
struct FFilterPresetHelpers
{
	/** Creates a new filtering preset according to the specific object names */
	static void CreateNewPreset(const TArray<TSharedPtr<ITraceObject>>& InObjects);
	/** Creates a set of strings, corresponding to set of non-filtered out object as part of InObjects */
	static void ExtractEnabledObjectNames(const TArray<TSharedPtr<ITraceObject>>& InObjects, TArray<FString>& OutNames);
	/** Returns whether or not shared presets can be modified, requires write-flag on default confing files */
	static bool CanModifySharedPreset();
};
}

/** Structure representing an individual preset in configuration (ini) files */
USTRUCT()
struct FTraceFilterData
{
	GENERATED_BODY()

	UPROPERTY()
	FString Name;

	UPROPERTY()
	TArray<FString> AllowlistedNames;

	bool operator==(const FTraceFilterData& Other) const
	{
		return Name == Other.Name && AllowlistedNames == Other.AllowlistedNames;
	}
};

/** UObject containers for the preset data */
UCLASS(Config = Engine)
class ULocalTraceFilterPresetContainer : public UObject
{
	GENERATED_BODY()

	friend struct UE::TraceTools::FFilterPresetHelpers;
public:
	void GetUserPresets(TArray<TSharedPtr<UE::TraceTools::ITraceFilterPreset>>& OutPresets);

	static void AddFilterData(const FTraceFilterData& InFilterData);
	static bool RemoveFilterData(const FTraceFilterData& InFilterData);
	static void Save();
protected:
	UPROPERTY(Config)
	TArray<FTraceFilterData> UserPresets;
};

UCLASS(Config = Engine, DefaultConfig)
class USharedTraceFilterPresetContainer : public UObject
{
	GENERATED_BODY()

	friend struct UE::TraceTools::FFilterPresetHelpers;
public:
	void GetSharedUserPresets(TArray<TSharedPtr<UE::TraceTools::ITraceFilterPreset>>& OutPresets);

	static void AddFilterData(const FTraceFilterData& InFilterData);
	static bool RemoveFilterData(const FTraceFilterData& InFilterData);
	static void Save();
protected:
	UPROPERTY(Config)
	TArray<FTraceFilterData> SharedPresets;
};

namespace UE::TraceTools
{

/** Base implementation of a filter preset */
struct FFilterPresetBase : public ITraceFilterPreset
{
protected:
	FFilterPresetBase(const FString& InName) : Name(InName) {}
	virtual FText GetDescription() const;

protected:
	FString Name;
};

struct FFilterPreset : public FFilterPresetBase
{
public:
	FFilterPreset(const FString& InName, FTraceFilterData& InFilterData) : FFilterPresetBase(InName), FilterData(InFilterData) {}

	/** Begin ITraceFilterPreset overrides */
	virtual FString GetName() const override;
	virtual FText GetDisplayText() const;
	virtual void GetAllowlistedNames(TArray<FString>& OutNames) const override;
	virtual bool CanDelete() const override;
	virtual void Rename(const FString& InNewName) override;
	virtual bool Delete() override;
	virtual bool MakeShared() override;
	virtual bool MakeLocal() override;
	virtual bool IsLocal() const override;
	virtual bool IsEnginePreset() const override { return false; }
	virtual void Save(const TArray<TSharedPtr<TraceTools::ITraceObject>>& InObjects) override {}
	virtual void Save() override {}
	/** End ITraceFilterPreset overrides */
protected:
	FTraceFilterData& FilterData;
};

// The Engine presets don't use USTRUCT so they can be shared with FTraceAuxiliary.
struct FEngineFilterPreset : public FFilterPresetBase
{
public:
	FEngineFilterPreset(const FString& InName, const TArray<FString>& InAllowListedNames) : FFilterPresetBase(InName), AllowListedNames(InAllowListedNames) {}

	/** Begin ITraceFilterPreset overrides */
	virtual FString GetName() const override { return Name; }
	virtual FText GetDisplayText() const { return FText::FromString(Name); }
	virtual void GetAllowlistedNames(TArray<FString>& OutNames) const override;
	virtual bool CanDelete() const override { return false; }
	virtual void Rename(const FString& InNewName) override {}
	virtual bool Delete() override { return false; }
	virtual bool MakeShared() override { return false; }
	virtual bool MakeLocal() override { return false; }
	virtual bool IsLocal() const override { return false; }
	virtual bool IsEnginePreset() const override { return true; }
	virtual void Save(const TArray<TSharedPtr<TraceTools::ITraceObject>>& InObjects) override {}
	virtual void Save() override {}
	/** End ITraceFilterPreset overrides */

protected:
	TArray<FString> AllowListedNames;
};

/** User filter preset, allows for deletion / transitioning INI ownership */
struct FUserFilterPreset : public FFilterPreset
{
public:
	FUserFilterPreset(const FString& InName, FTraceFilterData& InFilterData, bool bInLocal = false) : FFilterPreset(InName, InFilterData), bIsLocalPreset(bInLocal) {}

	/** Begin ITraceFilterPreset overrides */
	virtual bool CanDelete() const override;
	virtual bool Delete() override;
	virtual bool MakeShared() override;
	virtual bool MakeLocal() override;
	virtual bool IsLocal() const override;
	virtual void Save(const TArray<TSharedPtr<TraceTools::ITraceObject>>& InObjects) override;
	virtual void Save() override;
	/** End ITraceFilterPreset overrides */
protected:
	bool bIsLocalPreset;
};

} // namespace UE::TraceTools