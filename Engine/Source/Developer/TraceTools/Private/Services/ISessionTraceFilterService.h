// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

// TraceTools
#include "ITraceController.h"

struct FDateTime;

namespace UE::TraceTools
{

struct ITraceFilterPreset;

struct FTraceObjectInfo
{
	FString Name;
	FString Description;
	bool bEnabled;
	bool bReadOnly;
	uint32 Id;

	bool operator<(const FTraceObjectInfo& Other) const
	{
		return Name.Compare(Other.Name) < 0;
	}
};

struct FTraceStats
{
	FTraceStatus::FStats StandardStats;

	//Computed stats
	uint64 BytesSentPerSecond = 0;
	uint64 BytesTracedPerSecond = 0;
};

/** Filtering service, representing the state and data for a specific TraceServices::IAnalysisSession */
class ISessionTraceFilterService : public TSharedFromThis<ISessionTraceFilterService>
{
public:
	virtual ~ISessionTraceFilterService() = default;

	/** Returns the root level set of objects. */
	virtual void GetRootObjects(TArray<FTraceObjectInfo>& OutObjects) const = 0;
	
	/** Returns the object with the specified name. */
	virtual const FTraceObjectInfo* GetObject(const FString& Name ) const = 0;

	/** Set the filtered state for an individual object by its hash. */
	virtual void SetObjectFilterState(const FString& InObjectName, const bool bFilterState) = 0;
	
	/** Get the timestamp for the last channel data update. */
	virtual const FDateTime& GetChannelsUpdateTimestamp() const = 0;

	/** Update filter preset */
	virtual void UpdateFilterPreset(const TSharedPtr<ITraceFilterPreset> InPreset, bool IsEnabled) = 0;

	/** Returns true if settings are available for the selected session. */
	virtual bool HasSettings() const = 0 ;

	/** Get the settings of the selected session. */
	virtual const FTraceStatus::FSettings& GetSettings() const = 0;

	/** Returns true if stats are available for the selected session. */
	virtual bool HasStats() const = 0;

	/** Get the stats of the selected session. */
	virtual const FTraceStats& GetStats() const = 0;

	/** Get the endpoint of the current running trace. */
	virtual const FString& GetTraceEndpoint() const = 0;

	/** Get the current status of the trace system. */
	virtual FTraceStatus::ETraceSystemStatus GetTraceSystemStatus() const = 0;

	/** Sets the InstanceId to control. Supports invalid guid value for disabled state. */
	virtual void SetInstanceId(const FGuid& Id) = 0;

	/** Returns true if the session with the currently set InstanceId is available for communication. */
	virtual bool HasAvailableInstance() const = 0;
};

} // namespace UE::TraceTools
