// Copyright Epic Games, Inc. All Rights Reserved.

#include "SessionTraceControllerFilterService.h"

#include "Hash/xxhash.h"
#include "Misc/CoreDelegates.h"

// TraceTools
#include "Models/ITraceFilterPreset.h"

namespace UE::TraceTools
{

uint64 HashName(FStringView Name)
{
	// Strip plurals and convert to upper case.
	TStringBuilder<48> TransformedBuffer;
	TransformedBuffer << (Name.EndsWith('s') ? Name.LeftChop(1) : Name);
	FCString::Strupr(TransformedBuffer.GetData(), TransformedBuffer.Len());

	return FXxHash64::HashBuffer(TransformedBuffer.GetData(), TransformedBuffer.Len() * sizeof(TCHAR)).Hash;
}

FSessionTraceControllerFilterService::FSessionTraceControllerFilterService(TSharedPtr<ITraceController> InTraceController)
{
	FCoreDelegates::OnEndFrame.AddRaw(this, &FSessionTraceControllerFilterService::OnApplyChannelChanges);

	TraceController = InTraceController;
	TraceController->OnStatusReceived().AddRaw(this, &FSessionTraceControllerFilterService::OnTraceStatusUpdated);
}

FSessionTraceControllerFilterService::~FSessionTraceControllerFilterService()
{
	FCoreDelegates::OnEndFrame.RemoveAll(this);
	TraceController->OnStatusReceived().RemoveAll(this);
}

void FSessionTraceControllerFilterService::GetRootObjects(TArray<FTraceObjectInfo>& OutObjects) const
{
	for (auto& Pair : Objects)
	{
		OutObjects.Add(Pair.Value);
	}
}

const FTraceObjectInfo* FSessionTraceControllerFilterService::GetObject(const FString& Name) const
{
	return Objects.Find(HashName(Name));
}

const FDateTime& FSessionTraceControllerFilterService::GetChannelsUpdateTimestamp() const
{
	return ChannelsTimestamp;
}

void FSessionTraceControllerFilterService::SetObjectFilterState(const FString& InObjectName, const bool bFilterState)
{
	if (bFilterState)
	{
		FrameDisabledChannels.Remove(InObjectName);
		FrameEnabledChannels.Add(InObjectName);
	}
	else
	{
		FrameEnabledChannels.Remove(InObjectName);
		FrameDisabledChannels.Add(InObjectName);
	}
}

void FSessionTraceControllerFilterService::UpdateFilterPreset(const TSharedPtr<ITraceFilterPreset> Preset, bool IsEnabled)
{
	TArray<FString> Names;
	Preset->GetAllowlistedNames(Names);
	if (IsEnabled)
	{
		FrameEnabledChannels.Append(Names);

		for (const FString& Name : Names)
		{
			FrameDisabledChannels.Remove(Name);
		}
	}
	else
	{
		FrameDisabledChannels.Append(Names);

		for (const FString& Name : Names)
		{
			FrameEnabledChannels.Remove(Name);
		}
	}
}

void FSessionTraceControllerFilterService::DisableAllChannels()
{
	for (auto& ObjectInfo : Objects)
	{
		ObjectInfo.Value.bEnabled = false;
	}
}

void FSessionTraceControllerFilterService::OnTraceStatusUpdated(const FTraceStatus& InStatus, FTraceStatus::EUpdateType InUpdateType, ITraceControllerCommands& Commands)
{
	if (!InstanceId.IsValid() || InStatus.InstanceId != InstanceId)
	{
		return;
	}

	if (!TraceController->HasAvailableInstance(InstanceId))
	{
		return;
	}

	if (EnumHasAnyFlags(InUpdateType, FTraceStatus::EUpdateType::ChannelsDesc) ||
		(EnumHasAnyFlags(InUpdateType, FTraceStatus::EUpdateType::ChannelsStatus)))
	{
		UpdateChannels(InStatus);
	}

	if (EnumHasAnyFlags(InUpdateType, FTraceStatus::EUpdateType::Settings))
	{
		Settings = InStatus.Settings;
		bHasSettings = true;
	}

	if (EnumHasAnyFlags(InUpdateType, FTraceStatus::EUpdateType::Status))
	{
		TraceEndpoint = InStatus.Endpoint;
		TraceSystemStatus = InStatus.TraceSystemStatus;

		Stats.BytesSentPerSecond = 0;
		Stats.BytesTracedPerSecond = 0;

		double DeltaTimeSeconds = (InStatus.StatusTimestamp - StatusTimestamp).GetTotalSeconds();

		if (DeltaTimeSeconds > 0.0f)
		{
			Stats.BytesSentPerSecond = FMath::Max(0ull, InStatus.Stats.BytesSent - Stats.StandardStats.BytesSent);
			Stats.BytesSentPerSecond = (uint64) ((double)Stats.BytesSentPerSecond / DeltaTimeSeconds);

			Stats.BytesTracedPerSecond = FMath::Max(0ull, InStatus.Stats.BytesTraced - Stats.StandardStats.BytesTraced);
			Stats.BytesTracedPerSecond = (uint64) ((double)Stats.BytesTracedPerSecond / DeltaTimeSeconds);
		}

		Stats.StandardStats = InStatus.Stats;
		StatusTimestamp = InStatus.StatusTimestamp;
		bHasStats = true;
	}
}

void FSessionTraceControllerFilterService::UpdateChannels(const FTraceStatus& InStatus)
{
	const TMap<uint32, FTraceStatus::FChannel>& Channels = InStatus.Channels;

	if (Channels.Num())
	{
		bChannelsReceived = true;
	}

	bool bChanged = bHasPendingChannelChanges || Channels.Num() != Objects.Num();
	bHasPendingChannelChanges = false;

	TMap<uint64, FTraceObjectInfo> NewObjects;
	NewObjects.Reserve(Channels.Num());

	for (const auto& Entry : Channels)
	{
		const uint64 Hash = HashName(Entry.Value.Name);
		FTraceObjectInfo& NewInfo = NewObjects.Add(Hash);

		if (!bChanged)
		{
			if (const FTraceObjectInfo* Existing = Objects.Find(Hash))
			{
				if (Existing->bEnabled != Entry.Value.bEnabled)
				{
					bChanged = true;
				}
			}
			else
			{
				bChanged = true;
			}
		}

		NewInfo.Name = Entry.Value.Name;
		NewInfo.Description = Entry.Value.Description;
		NewInfo.bEnabled = Entry.Value.bEnabled;
		NewInfo.bReadOnly = Entry.Value.bReadOnly;
		NewInfo.Id = Entry.Value.Id;
	}

	if (bChanged)
	{
		ChannelsTimestamp = FDateTime::Now();
		Objects = MoveTemp(NewObjects);
	}
}

void FSessionTraceControllerFilterService::OnApplyChannelChanges()
{
	if (!TraceController->HasAvailableInstance(InstanceId) || !bChannelsReceived)
	{
		return;
	}

	if (FrameEnabledChannels.Num() || FrameDisabledChannels.Num())
	{
		TraceController->WithInstance(InstanceId, [&](const FTraceStatus& Status, ITraceControllerCommands& Commands)
		{
			Commands.SetChannels(FrameEnabledChannels.Array(), FrameDisabledChannels.Array());
		});

		FrameEnabledChannels.Empty();
		FrameDisabledChannels.Empty();

		bHasPendingChannelChanges = true;
	}
}

bool FSessionTraceControllerFilterService::HasSettings() const
{
	return bHasSettings;
}

const FTraceStatus::FSettings& FSessionTraceControllerFilterService::GetSettings() const
{
	return Settings;
}

bool FSessionTraceControllerFilterService::HasStats() const
{
	return bHasStats;
}

const FTraceStats& FSessionTraceControllerFilterService::GetStats() const
{
	return Stats;
}

void FSessionTraceControllerFilterService::Reset()
{
	Objects.Empty(Objects.Num());

	bHasStats = false;
	bHasSettings = false;
	TraceEndpoint.Empty();
	ChannelsTimestamp = FDateTime();
	StatusTimestamp = FDateTime();
	bHasPendingChannelChanges = false;
}

void FSessionTraceControllerFilterService::SetInstanceId(const FGuid& Id)
{
	InstanceId = Id;
	Reset();
}

bool FSessionTraceControllerFilterService::HasAvailableInstance() const
{
	if (InstanceId.IsValid())
	{
		return TraceController->HasAvailableInstance(InstanceId);
	}

	return false;
}

} // namespace UE::TraceTools