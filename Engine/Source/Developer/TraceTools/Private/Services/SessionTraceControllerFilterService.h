// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISessionTraceFilterService.h"
#include "Misc/DateTime.h"

namespace TraceServices
{
	class IAnalysisSession;
	typedef uint64 FSessionHandle;
}

namespace UE::TraceTools
{

class IEventInfoProvider;

/** Implementation of ISessionTraceFilterService to query and set channels using the TraceController (MessageBus). */
class FSessionTraceControllerFilterService : public ISessionTraceFilterService
{
public:
	FSessionTraceControllerFilterService(TSharedPtr<ITraceController> InTraceController);
	virtual ~FSessionTraceControllerFilterService();

	/** Begin ISessionTraceFilterService overrides */
	virtual void GetRootObjects(TArray<FTraceObjectInfo>& OutObjects) const override;
	virtual const FTraceObjectInfo* GetObject(const FString& Name) const override;
	
	virtual const FDateTime& GetChannelsUpdateTimestamp() const override;
	virtual void SetObjectFilterState(const FString& InObjectName, const bool bFilterState) override;
	virtual void UpdateFilterPreset(const TSharedPtr<ITraceFilterPreset> InPreset, bool IsEnabled) override;

	virtual bool HasSettings() const override;
	virtual const FTraceStatus::FSettings& GetSettings() const override;

	virtual bool HasStats() const override;
	virtual const FTraceStats& GetStats() const override;

	virtual const FString& GetTraceEndpoint() const override { return TraceEndpoint; }
	virtual FTraceStatus::ETraceSystemStatus GetTraceSystemStatus() const { return TraceSystemStatus; }

	void SetInstanceId(const FGuid& Id) override;
	bool HasAvailableInstance() const override;

	/** End ISessionTraceFilterService overrides */

protected:
	/** Callback at end of engine frame, used to dispatch all enabled/disabled channels */
	void OnApplyChannelChanges();

	/** Retrieves channels names from provider and marks them all as disabled */
	void DisableAllChannels();

	void OnTraceStatusUpdated(const FTraceStatus& InStatus, FTraceStatus::EUpdateType InUpdateType, ITraceControllerCommands& Commands);

	void UpdateChannels(const FTraceStatus& InStatus);

	void Reset();

protected:
	TSharedPtr<ITraceController> TraceController;

	/** A map with the key formed by hashing the object name and the object as the value.*/
	TMap<uint64, FTraceObjectInfo> Objects;

	/** Names of channels that were either enabled or disabled during the duration of this frame */
	TSet<FString> FrameEnabledChannels;
	TSet<FString> FrameDisabledChannels;

	/** Timestamp at which the trace status was last updated */
	FDateTime StatusTimestamp;

	/** Timestamp at which the trace channels were last updated */
	FDateTime ChannelsTimestamp;

	bool bChannelsReceived = false;

	FTraceStatus::FSettings Settings;
	bool bHasSettings = false;

	FTraceStats Stats;
	bool bHasStats = false;

	FString TraceEndpoint;
	FTraceStatus::ETraceSystemStatus TraceSystemStatus;

	FGuid InstanceId;

	bool bHasPendingChannelChanges = false;
};

} // namespace UE::TraceTools