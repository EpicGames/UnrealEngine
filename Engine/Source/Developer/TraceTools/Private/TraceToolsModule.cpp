// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceToolsModule.h"

#include "Modules/ModuleManager.h"
#include "Misc/ConfigContext.h"
#include "TraceToolsStyle.h"
#include "Widgets/Docking/SDockTab.h"

// TraceTools
#include "Widgets/STraceControl.h"

IMPLEMENT_MODULE(FTraceToolsModule, TraceTools);

FString FTraceToolsModule::TraceFiltersIni;

void FTraceToolsModule::StartupModule()
{
	LLM_SCOPE_BYNAME(TEXT("Insights/TraceTools"));

	UE::TraceTools::FTraceToolsStyle::Initialize();

	TraceFiltersIni = GEngineIni;
}

void FTraceToolsModule::ShutdownModule()
{
	LLM_SCOPE_BYNAME(TEXT("Insights/TraceTools"));

	UE::TraceTools::FTraceToolsStyle::Shutdown();
}

TSharedRef<SWidget> FTraceToolsModule::CreateTraceControlWidget(TSharedPtr<ITraceController> InTraceController)
{
	return SNew(UE::TraceTools::STraceControl, InTraceController)
		   .AutoDetectSelectedSession(true);
}

TSharedRef<SWidget> FTraceToolsModule::CreateTraceControlWidget(TSharedPtr<ITraceController> InTraceController, FGuid InstanceId)
{
	TSharedRef<UE::TraceTools::STraceControl> TraceControlWidget = SNew(UE::TraceTools::STraceControl, InTraceController);
	TraceControlWidget->SetInstanceId(InstanceId);

	return TraceControlWidget;
}

void FTraceToolsModule::SetTraceControlWidgetInstanceId(TSharedRef<SWidget> Widget, FGuid InstanceId)
{
	TSharedRef<UE::TraceTools::STraceControl> TraceControl = StaticCastSharedRef<UE::TraceTools::STraceControl>(Widget);
	TraceControl->SetInstanceId(InstanceId);
}