// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceTools/Interfaces/ITraceToolsModule.h"

#include "Modules/ModuleInterface.h"

class FName;
class FString;
class SWidget;

class FTraceToolsModule : public UE::TraceTools::ITraceToolsModule
{
public:
	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	virtual TSharedRef<SWidget> CreateTraceControlWidget(TSharedPtr<ITraceController> InTraceController) override;
	virtual TSharedRef<SWidget> CreateTraceControlWidget(TSharedPtr<ITraceController> InTraceController, FGuid InstanceId) override;

	virtual void SetTraceControlWidgetInstanceId(TSharedRef<SWidget> Widget, FGuid InstanceId) override;

	static FString TraceFiltersIni;
};