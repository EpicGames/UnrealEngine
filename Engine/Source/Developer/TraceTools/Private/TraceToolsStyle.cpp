// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceToolsStyle.h"

#include "Brushes/SlateBoxBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Brushes/SlateNoResource.h"
#include "Misc/Paths.h"
#include "Styling/CoreStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/ToolBarStyle.h"

namespace UE::TraceTools
{

// This is to fix the issue that SlateStyleMacros like IMAGE_BRUSH look for RootToContentDir but StyleSet->RootToContentDir is how this style is set up
#define RootToContentDir StyleSet->RootToContentDir
#define RootToCoreContentDir StyleSet->RootToCoreContentDir

TSharedPtr< FSlateStyleSet > FTraceToolsStyle::StyleSet = nullptr;

FTextBlockStyle FTraceToolsStyle::NormalText;

// Const icon sizes
static const FVector2D Icon8x8(8.0f, 8.0f);
static const FVector2D Icon9x19(9.0f, 19.0f);
static const FVector2D Icon14x14(14.0f, 14.0f);
static const FVector2D Icon16x16(16.0f, 16.0f);
static const FVector2D Icon20x20(20.0f, 20.0f);
static const FVector2D Icon22x22(22.0f, 22.0f);
static const FVector2D Icon24x24(24.0f, 24.0f);
static const FVector2D Icon28x28(28.0f, 28.0f);
static const FVector2D Icon27x31(27.0f, 31.0f);
static const FVector2D Icon26x26(26.0f, 26.0f);
static const FVector2D Icon32x32(32.0f, 32.0f);
static const FVector2D Icon40x40(40.0f, 40.0f);
static const FVector2D Icon48x48(48.0f, 48.0f);
static const FVector2D Icon75x82(75.0f, 82.0f);
static const FVector2D Icon360x32(360.0f, 32.0f);
static const FVector2D Icon171x39(171.0f, 39.0f);
static const FVector2D Icon170x50(170.0f, 50.0f);
static const FVector2D Icon267x140(170.0f, 50.0f);

void FTraceToolsStyle::Initialize()
{
	LLM_SCOPE_BYNAME(TEXT("Insights/TraceTools"));

	// Only register once
	if( StyleSet.IsValid() )
	{
		return;
	}

	StyleSet = MakeShareable(new FSlateStyleSet("TraceToolsStyle") );
	
	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	StyleSet->Set("EventFilter.GroupBorder", new BOX_BRUSH("Common/GroupBorder", FMargin(4.0f / 16.0f)));

	NormalText = FTextBlockStyle()
		.SetFont(DEFAULT_FONT("Regular", FCoreStyle::RegularTextSize))
		.SetColorAndOpacity(FSlateColor::UseForeground())
		.SetShadowOffset(FVector2D::ZeroVector)
		.SetShadowColorAndOpacity(FLinearColor::Black)
		.SetHighlightColor(FLinearColor(0.02f, 0.3f, 0.0f))
		.SetHighlightShape(BOX_BRUSH("Common/TextBlockHighlightShape", FMargin(3.f / 8.f)));

	// Colors
	{
		StyleSet->Set("EventFilter.EnginePreset", FLinearColor(0.728f, 0.364f, 0.003f));
		StyleSet->Set("EventFilter.SharedPreset", FLinearColor(0.003f, 0.364f, 0.728f));
		StyleSet->Set("EventFilter.LocalPreset", FLinearColor(0.003f, 0.728f, 0.364f));
	}
	
	// Icons
	{
		StyleSet->Set("EventFilter.State.Enabled", new IMAGE_BRUSH("Common/CheckBox_Checked", Icon16x16));
		StyleSet->Set("EventFilter.State.Enabled_Hovered", new IMAGE_BRUSH("Common/CheckBox_Checked_Hovered", Icon16x16));

		StyleSet->Set("EventFilter.State.Disabled", new IMAGE_BRUSH("Common/CheckBox", Icon16x16));
		StyleSet->Set("EventFilter.State.Disabled_Hovered", new IMAGE_BRUSH("Common/CheckBox_Hovered", Icon16x16));

		StyleSet->Set("EventFilter.State.Pending", new IMAGE_BRUSH("Common/CheckBox_Undetermined", Icon16x16));
		StyleSet->Set("EventFilter.State.Pending_Hovered", new IMAGE_BRUSH("Common/CheckBox_Undetermined_Hovered", Icon16x16));

		StyleSet->Set("EventFilter.TabIcon", new IMAGE_BRUSH_SVG("Starship/Common/TraceDataFiltering", Icon16x16));
	}

	FButtonStyle Button = FButtonStyle()
		.SetNormal(BOX_BRUSH("Common/Button", FVector2D(32, 32), 8.0f / 32.0f))
		.SetHovered(BOX_BRUSH("Common/Button_Hovered", FVector2D(32, 32), 8.0f / 32.0f))
		.SetPressed(BOX_BRUSH("Common/Button_Pressed", FVector2D(32, 32), 8.0f / 32.0f))
		.SetNormalPadding(FMargin(2, 2, 2, 2))
		.SetPressedPadding(FMargin(2, 3, 2, 1));

	StyleSet->Set("ToggleButton", FButtonStyle(Button)
		.SetNormal(FSlateNoResource())
		.SetHovered(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, FLinearColor(0.701f, 0.225f, 0.003f)))
		.SetPressed(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, FLinearColor(0.701f, 0.225f, 0.003f)))
	);

	FComboButtonStyle ToolbarComboButton = FComboButtonStyle()
		.SetButtonStyle(StyleSet->GetWidgetStyle<FButtonStyle>("ToggleButton"))
		.SetDownArrowImage(IMAGE_BRUSH("Common/ShadowComboArrow", Icon8x8))
		.SetMenuBorderBrush(BOX_BRUSH("Old/Menu_Background", FMargin(8.0f / 64.0f)))
		.SetMenuBorderPadding(FMargin(0.0f));
	StyleSet->Set("EventFilter.ComboButton", ToolbarComboButton);
	
	StyleSet->Set("EventFilter.TextStyle", FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Bold", 9))
		.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f, 0.9f))
		.SetShadowOffset(FVector2D(1, 1))
		.SetShadowColorAndOpacity(FLinearColor(0, 0, 0, 0.9f)));

	// TraceControlToolbar
	{
		FToolBarStyle TraceControlToolbarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("SlimToolBar");

		TraceControlToolbarStyle.SetBackgroundPadding(FMargin(4.0f, 4.0f));
		TraceControlToolbarStyle.SetBlockPadding(FMargin(2.0f, 2.0f));
		TraceControlToolbarStyle.SetButtonPadding(FMargin(2.0f, 2.0f));
		TraceControlToolbarStyle.SetCheckBoxPadding(FMargin(2.0f, 2.0f));
		TraceControlToolbarStyle.SetComboButtonPadding(FMargin(2.0f, 2.0f));
		TraceControlToolbarStyle.SetIndentedBlockPadding(FMargin(2.0f, 2.0f));
		TraceControlToolbarStyle.SetLabelPadding(FMargin(2.0f, 2.0f));
		
		TraceControlToolbarStyle.ToggleButton.SetPadding(FMargin(2.0f, 2.0f));

		TraceControlToolbarStyle.ButtonStyle.SetNormalPadding(FMargin(6.0f, 2.0f, 4.0f, 2.0f));
		TraceControlToolbarStyle.ButtonStyle.SetPressedPadding(FMargin(6.0f, 2.0f, 4.0f, 2.0f));

		TraceControlToolbarStyle.IconSize.Set(20.0f, 20.0f);

		StyleSet->Set("TraceControlToolbar", TraceControlToolbarStyle);

		StyleSet->Set("TraceControlToolbar.MinUniformToolbarSize", 40.0f);
		StyleSet->Set("TraceControlToolbar.MaxUniformToolbarSize", 40.0f);
	}

	// Filter list
	/* Set images for various SCheckBox states associated with "ContentBrowser.FilterButton" ... */
	const FCheckBoxStyle FilterButtonCheckBoxStyle = FCheckBoxStyle()
		.SetUncheckedImage(IMAGE_BRUSH("ContentBrowser/FilterUnchecked", FVector2D(10.0f, 20.0f)))
		.SetUncheckedHoveredImage(IMAGE_BRUSH("ContentBrowser/FilterUnchecked", FVector2D(10.0f, 20.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
		.SetUncheckedPressedImage(IMAGE_BRUSH("ContentBrowser/FilterUnchecked", FVector2D(10.0f, 20.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
		.SetCheckedImage(IMAGE_BRUSH("ContentBrowser/FilterChecked", FVector2D(10.0f, 20.0f)))
		.SetCheckedHoveredImage(IMAGE_BRUSH("ContentBrowser/FilterChecked", FVector2D(10.0f, 20.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
		.SetCheckedPressedImage(IMAGE_BRUSH("ContentBrowser/FilterChecked", FVector2D(10.0f, 20.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)));
	/* ... and add the new style */
	StyleSet->Set("FilterPresets.FilterButton", FilterButtonCheckBoxStyle);

	StyleSet->Set("FilterPresets.FilterNameFont", DEFAULT_FONT("Regular", 10));
	StyleSet->Set("FilterPresets.FilterButtonBorder", new BOX_BRUSH("Common/RoundedSelection_16x", FMargin(4.0f / 16.0f)));

	StyleSet->Set("FilterPresets.TableBackground", new BOX_BRUSH("Common/TableViewMajorColumn", FMargin(4.0f / 16.0f)));
	StyleSet->Set("FilterPresets.SessionWarningBorder", new BOX_BRUSH("Common/GroupBorder", FMargin(4.0f / 16.0f)));
	StyleSet->Set("FilterPresets.BackgroundBorder", new BOX_BRUSH("Common/DarkGroupBorder", FMargin(4.0f / 16.0f)));
	StyleSet->Set("FilterPresets.WarningIcon", new CORE_IMAGE_BRUSH_SVG("Starship/Common/alert-circle", Icon40x40, FStyleColors::Warning));

	StyleSet->Set("FontAwesome.9", REGULAR_ICON_FONT(9));

	StyleSet->Set("TraceControl.StartTrace", new CORE_IMAGE_BRUSH_SVG("Starship/Insights/TraceTools/TraceStart", Icon40x40));
	StyleSet->Set("TraceControl.StartTrace.Small", new CORE_IMAGE_BRUSH_SVG("Starship/Insights/TraceTools/TraceStart", Icon20x20));

	StyleSet->Set("TraceControl.PauseTrace", new CORE_IMAGE_BRUSH_SVG("Starship/Insights/TraceTools/TracePause", Icon40x40));
	StyleSet->Set("TraceControl.PauseTrace.Small", new CORE_IMAGE_BRUSH_SVG("Starship/Insights/TraceTools/TracePause", Icon20x20));

	StyleSet->Set("TraceControl.ResumeTrace", new CORE_IMAGE_BRUSH_SVG("Starship/Insights/TraceTools/TraceResume", Icon40x40));
	StyleSet->Set("TraceControl.ResumeTrace.Small", new CORE_IMAGE_BRUSH_SVG("Starship/Insights/TraceTools/TraceResume", Icon20x20));

	StyleSet->Set("TraceControl.TraceSnapshot", new CORE_IMAGE_BRUSH_SVG("Starship/Insights/TraceTools/TraceSnapshot", Icon40x40));
	StyleSet->Set("TraceControl.TraceSnapshot.Small", new CORE_IMAGE_BRUSH_SVG("Starship/Insights/TraceTools/TraceSnapshot", Icon20x20));

	StyleSet->Set("TraceControl.SetTraceTargetServer", new CORE_IMAGE_BRUSH_SVG("Starship/Common/server", Icon40x40));
	StyleSet->Set("TraceControl.SetTraceTargetServer.Small", new CORE_IMAGE_BRUSH_SVG("Starship/Common/server", Icon20x20));

	StyleSet->Set("TraceControl.SetTraceTargetFile", new CORE_IMAGE_BRUSH_SVG("Starship/Common/file", Icon40x40));
	StyleSet->Set("TraceControl.SetTraceTargetFile.Small", new CORE_IMAGE_BRUSH_SVG("Starship/Common/file", Icon20x20));

	StyleSet->Set("TraceControl.TraceScreenshot", new CORE_IMAGE_BRUSH_SVG("../Editor/Slate/Starship/Common/HighResolutionScreenshot", Icon40x40));
	StyleSet->Set("TraceControl.TraceScreenshot.Small", new CORE_IMAGE_BRUSH_SVG("../Editor/Slate/Starship/Common/HighResolutionScreenshot", Icon20x20));

	StyleSet->Set("TraceControl.TraceBookmark", new CORE_IMAGE_BRUSH_SVG("../Editor/Slate/Starship/Common/Bookmarks", Icon40x40));
	StyleSet->Set("TraceControl.TraceBookmark.Small", new CORE_IMAGE_BRUSH_SVG("../Editor/Slate/Starship/Common/Bookmarks", Icon20x20));

	StyleSet->Set("TraceControl.ToggleStatNamedEvents", new CORE_IMAGE_BRUSH_SVG("../Editor/Slate/Starship/Common/Statistics", Icon40x40));
	StyleSet->Set("TraceControl.ToggleStatNamedEvents.Small", new CORE_IMAGE_BRUSH_SVG("../Editor/Slate/Starship/Common/Statistics", Icon20x20));

	StyleSet->Set("ToggleTraceButton.RecordTraceCenter.StatusBar", new CORE_IMAGE_BRUSH_SVG("Starship/Insights/TraceTools/RecordTraceCenter", Icon16x16));
	StyleSet->Set("ToggleTraceButton.RecordTraceOutline.StatusBar", new CORE_IMAGE_BRUSH_SVG("Starship/Insights/TraceTools/RecordTraceOutline", Icon16x16));
	StyleSet->Set("ToggleTraceButton.RecordTraceRecording.StatusBar", new CORE_IMAGE_BRUSH_SVG("Starship/Insights/TraceTools/RecordTraceRecording", Icon16x16));
	StyleSet->Set("ToggleTraceButton.TraceStop.StatusBar", new CORE_IMAGE_BRUSH_SVG("Starship/Insights/TraceTools/TraceStop", Icon16x16, FStyleColors::Error));

	StyleSet->Set("ToggleTraceButton.RecordTraceCenter.SlimToolbar", new CORE_IMAGE_BRUSH_SVG("Starship/Insights/TraceTools/RecordTraceCenter", Icon20x20));
	StyleSet->Set("ToggleTraceButton.RecordTraceOutline.SlimToolbar", new CORE_IMAGE_BRUSH_SVG("Starship/Insights/TraceTools/RecordTraceOutline", Icon20x20));
	StyleSet->Set("ToggleTraceButton.RecordTraceRecording.SlimToolbar", new CORE_IMAGE_BRUSH_SVG("Starship/Insights/TraceTools/RecordTraceRecording", Icon20x20));
	StyleSet->Set("ToggleTraceButton.TraceStop.SlimToolbar", new CORE_IMAGE_BRUSH_SVG("Starship/Insights/TraceTools/TraceStop", Icon20x20, FStyleColors::Error));
	
	StyleSet->Set("TraceStatistics.CopyEndpoint", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Copy", FVector2D(10.0f, 10.0f)));
	
	FSlateStyleRegistry::RegisterSlateStyle( *StyleSet.Get() );
}

#undef RootToContentDir
#undef RootToCoreContentDir

void FTraceToolsStyle::Shutdown()
{
	if( StyleSet.IsValid() )
	{
		FSlateStyleRegistry::UnRegisterSlateStyle( *StyleSet.Get() );
		ensure( StyleSet.IsUnique() );
		StyleSet.Reset();
	}
}

const ISlateStyle& FTraceToolsStyle::Get()
{
	return *( StyleSet.Get() );
}

const FName& FTraceToolsStyle::GetStyleSetName()
{
	return StyleSet->GetStyleSetName();
}

} // namespace UE::TraceTools