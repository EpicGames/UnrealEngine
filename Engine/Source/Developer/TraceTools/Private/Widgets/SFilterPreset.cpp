// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFilterPreset.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

// TraceTools
#include "Models/ITraceFilterPreset.h"
#include "TraceToolsStyle.h"

#define LOCTEXT_NAMESPACE "SFilterPreset"

namespace UE::TraceTools
{

SFilterPreset::~SFilterPreset()
{
	if (IsHovered())
	{
		OnHighlightPreset.ExecuteIfBound(nullptr);
	}
}

void SFilterPreset::Construct(const FArguments& InArgs)
{
	bEnabled = false;
	bHighlighted = false;
	OnPresetChanged = InArgs._OnPresetChanged;
	OnRequestRemove = InArgs._OnRequestRemove;
	OnRequestEnableOnly = InArgs._OnRequestEnableOnly;
	OnRequestEnableAll = InArgs._OnRequestEnableAll;
	OnRequestDisableAll = InArgs._OnRequestDisableAll;
	OnRequestRemoveAll = InArgs._OnRequestRemoveAll;
	OnRequestDelete = InArgs._OnRequestDelete;
	OnRequestSave = InArgs._OnRequestSave;
	OnHighlightPreset = InArgs._OnHighlightPreset;

	FilterPreset = InArgs._FilterPreset;

	const FName ColorName = [this]()
	{
		if (FilterPreset->CanDelete())
		{
			// User preset 
			if (FilterPreset->IsLocal())
			{
				return FName("EventFilter.LocalPreset");
			}
			
			return FName("EventFilter.SharedPreset");
		}
		
		// Engine preset
		return FName("EventFilter.EnginePreset");
	}();

	FilterColor = FTraceToolsStyle::Get().GetColor(ColorName);

	ChildSlot
	[
		SNew(SBorder)
		.Padding(0)
		.BorderBackgroundColor(FLinearColor(0.2f, 0.2f, 0.2f, 0.2f))
		.BorderImage(FTraceToolsStyle::GetBrush("FilterPresets.FilterButtonBorder"))
		[
			SAssignNew(ToggleButtonPtr, SFilterPresetCheckBox)
			.Style(FTraceToolsStyle::Get(), "FilterPresets.FilterButton")
			.Padding(this, &SFilterPreset::GetNameLabelPadding)
			.IsChecked(this, &SFilterPreset::IsChecked)
			.OnCheckStateChanged(this, &SFilterPreset::PresetToggled)
			.OnGetMenuContent(this, &SFilterPreset::GetRightClickMenuContent)
			.ForegroundColor(this, &SFilterPreset::GetPresetForegroundColor)
			[
				SNew(STextBlock)
				.ColorAndOpacity(this, &SFilterPreset::GetNameLabelColorAndOpacity)
				.Font(FTraceToolsStyle::Get().GetFontStyle("FilterPresets.FilterNameFont"))
				.ShadowOffset(FVector2D(1.f, 1.f))
				.Text(this, &SFilterPreset::GetPresetName)
			]
		]
	];

	ToggleButtonPtr->SetOnFilterCtrlClicked(FOnClicked::CreateSP(this, &SFilterPreset::FilterCtrlClicked));
	ToggleButtonPtr->SetOnFilterAltClicked(FOnClicked::CreateSP(this, &SFilterPreset::FilterAltClicked));
	ToggleButtonPtr->SetOnFilterDoubleClicked(FOnClicked::CreateSP(this, &SFilterPreset::FilterDoubleClicked));
	ToggleButtonPtr->SetOnFilterMiddleButtonClicked(FOnClicked::CreateSP(this, &SFilterPreset::FilterMiddleButtonClicked));

	TAttribute<FText> Attribute = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SFilterPreset::GetToolTipText));
	ToggleButtonPtr->SetToolTipText(Attribute);
}

void SFilterPreset::SetEnabled(bool InEnabled)
{
	if (InEnabled != bEnabled)
	{
		bEnabled = InEnabled;
 		OnPresetChanged.ExecuteIfBound(*this);
	}
}

bool SFilterPreset::IsEnabled() const
{
	return bEnabled;
}

void SFilterPreset::MarkAsEnabled(bool InEnabled)
{
	bEnabled = InEnabled;
}

const TSharedPtr<ITraceFilterPreset>& SFilterPreset::GetFilterPreset() const
{
	return FilterPreset;
}

void SFilterPreset::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	SCompoundWidget::OnMouseLeave(MouseEvent);
	
	if (bHighlighted)
	{
		OnHighlightPreset.ExecuteIfBound(nullptr);
	}
	
	bHighlighted = false;
}

FReply SFilterPreset::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (IsHovered())
	{
		const bool bWasHighlighted = bHighlighted;
		bHighlighted = MouseEvent.IsControlDown();
		if (bHighlighted || bWasHighlighted)
		{
			OnHighlightPreset.ExecuteIfBound(bHighlighted ? FilterPreset : nullptr);
		}
	}

	return SCompoundWidget::OnMouseMove(MyGeometry, MouseEvent);
}

void SFilterPreset::PresetToggled(ECheckBoxState NewState)
{
	SetEnabled(NewState == ECheckBoxState::Checked);
}

FReply SFilterPreset::FilterCtrlClicked()
{
	OnRequestEnableAll.ExecuteIfBound();
	return FReply::Handled();
}

FReply SFilterPreset::FilterAltClicked()
{
	OnRequestDisableAll.ExecuteIfBound();
	return FReply::Handled();
}

FReply SFilterPreset::FilterDoubleClicked()
{
	// Disable all other presets and enable this one.
	OnRequestDisableAll.ExecuteIfBound();
	SetEnabled(true);
	return FReply::Handled();
}

FReply SFilterPreset::FilterMiddleButtonClicked()
{
	RemovePreset();
	return FReply::Handled();
}

FText SFilterPreset::GetToolTipText() const
{
	return bHighlighted ? FilterPreset->GetDescription() : FText::Format(LOCTEXT("FilterPresetTooltipText", "{0}\nHold CTRL to highlight allowlisted items."), FilterPreset->GetDescription());
}

TSharedRef<SWidget> SFilterPreset::GetRightClickMenuContent()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, NULL);

	MenuBuilder.BeginSection("PresetOptions", LOCTEXT("FilterContextHeading", "Preset Options"));
	{
		MenuBuilder.AddMenuEntry(
			FText::Format(LOCTEXT("RemovePreset", "Remove: {0}"), GetPresetName()),
			LOCTEXT("RemovePresetTooltip", "Remove this preset from the list. It can be added again in the presets menu."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SFilterPreset::RemovePreset))
		);

		MenuBuilder.AddMenuEntry(
			FText::Format(LOCTEXT("EnableOnlyThisPreset", "Enable this only: {0}"), GetPresetName()),
			LOCTEXT("EnableOnlyThisPresetTooltip", "Enable only this prest from the list."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SFilterPreset::EnableOnly))
		);

	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("PresetBulkOptions", LOCTEXT("BulkFilterContextHeading", "Bulk Preset Options"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("EnableAllPresets", "Enable All Presets"),
			LOCTEXT("EnableAllPresetsTooltip", "Enables all presets."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SFilterPreset::EnableAllPresets))
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("DisableAllPresets", "Disable All Presets"),
			LOCTEXT("DisableAllPresetsTooltip", "Disables all active presets."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SFilterPreset::DisableAllPresets))
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("RemoveAllPresets", "Remove All Presets"),
			LOCTEXT("RemoveAllPresetsTooltip", "Removes all presets from the list."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SFilterPreset::RemoveAllPresets))
		);
	}
	MenuBuilder.EndSection();


	MenuBuilder.BeginSection("UserPresetOptions", LOCTEXT("UserPresetsContextHeading", "User Presets"));
	{
		if (FilterPreset->CanDelete())
		{
			MenuBuilder.AddMenuEntry(
				FText::Format(LOCTEXT("DeletePreset", "Delete User Preset: {0}"), GetPresetName()),
				LOCTEXT("DeleteFilterTooltip", "Deletes this User Preset."),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SFilterPreset::DeletePreset))
			);

			MenuBuilder.AddMenuEntry(
				FText::Format(LOCTEXT("SavePreset", "Save User Preset: {0}"), GetPresetName()),
				LOCTEXT("SaveFilterTooltip", "Saves the currently filtering state as the selected User Preset."),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SFilterPreset::SavePreset))
			);
		}
	}
	MenuBuilder.EndSection();


	return MenuBuilder.MakeWidget();
}

void SFilterPreset::RemovePreset()
{
	TSharedRef<SFilterPreset> Self = SharedThis(this);
	OnRequestRemove.ExecuteIfBound(Self);
}

void SFilterPreset::EnableOnly()
{
	TSharedRef<SFilterPreset> Self = SharedThis(this);
	OnRequestEnableOnly.ExecuteIfBound(Self);
}

void SFilterPreset::EnableAllPresets()
{
	OnRequestEnableAll.ExecuteIfBound();
}

void SFilterPreset::DisableAllPresets()
{
	OnRequestDisableAll.ExecuteIfBound();
}

void SFilterPreset::RemoveAllPresets()
{
	OnRequestRemoveAll.ExecuteIfBound();
}

void SFilterPreset::SavePreset()
{
	TSharedRef<SFilterPreset> Self = SharedThis(this);
	OnRequestSave.ExecuteIfBound(Self);
}

void SFilterPreset::DeletePreset()
{
	TSharedRef<SFilterPreset> Self = SharedThis(this);
	OnRequestDelete.ExecuteIfBound(Self);
}

ECheckBoxState SFilterPreset::IsChecked() const
{
	return bEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

FSlateColor SFilterPreset::GetPresetForegroundColor() const
{
	return IsChecked() == ECheckBoxState::Checked ? FilterColor : FLinearColor::White;
}

FMargin SFilterPreset::GetNameLabelPadding() const
{
	return ToggleButtonPtr->IsPressed() ? FMargin(3, 2, 4, 0) : FMargin(3, 1, 4, 1);
}

FSlateColor SFilterPreset::GetNameLabelColorAndOpacity() const
{
	const float DimFactor = 0.75f;
	return IsHovered() ? FLinearColor(DimFactor, DimFactor, DimFactor, 1.0f) : FLinearColor::White;
}

FText SFilterPreset::GetPresetName() const
{
	FText PresetDisplayText = FilterPreset->GetDisplayText();

	if (PresetDisplayText.IsEmpty())
	{
		PresetDisplayText = LOCTEXT("UnknownPresetName", "???");
	}

	return PresetDisplayText;
}

} // namespace UE::TraceTools

#undef LOCTEXT_NAMESPACE
