// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Input/SCheckBox.h"

enum class ECheckBoxState : uint8;

namespace UE::TraceTools
{

struct ITraceFilterPreset;

DECLARE_DELEGATE_OneParam(FOnPresetChanged, const class SFilterPreset&);

/** A class for check boxes in the preset list. If you double click the checkbox, you will enable it and disable all others */
class SFilterPresetCheckBox : public SCheckBox
{
public:
	
	void SetOnFilterCtrlClicked(const FOnClicked& NewFilterCtrlClicked)
	{
		OnFilterCtrlClicked = NewFilterCtrlClicked;
	}

	void SetOnFilterAltClicked(const FOnClicked& NewFilteAltClicked)
	{
		OnFilterAltClicked = NewFilteAltClicked;
	}

	void SetOnFilterDoubleClicked(const FOnClicked& NewFilterDoubleClicked)
	{
		OnFilterDoubleClicked = NewFilterDoubleClicked;
	}

	void SetOnFilterMiddleButtonClicked(const FOnClicked& NewFilterMiddleButtonClicked)
	{
		OnFilterMiddleButtonClicked = NewFilterMiddleButtonClicked;
	}

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override
	{
		// Handle left mouse button double click
		if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && OnFilterDoubleClicked.IsBound())
		{
			return OnFilterDoubleClicked.Execute();
		}
		else
		{
			return SCheckBox::OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent);
		}
	}

	virtual FReply OnMouseButtonUp(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override
	{
		SCheckBox::OnMouseButtonUp(InMyGeometry, InMouseEvent);

		if (InMouseEvent.IsControlDown() && OnFilterCtrlClicked.IsBound())
		{
			return OnFilterCtrlClicked.Execute();
		}
		else if (InMouseEvent.IsAltDown() && OnFilterAltClicked.IsBound())
		{
			return OnFilterAltClicked.Execute();
		}
		else if (InMouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton && OnFilterMiddleButtonClicked.IsBound())
		{
			return OnFilterMiddleButtonClicked.Execute();
		}
		else
		{			
			return FReply::Handled().ReleaseMouseCapture();
		}
	}

private:
	FOnClicked OnFilterCtrlClicked;
	FOnClicked OnFilterAltClicked;
	FOnClicked OnFilterDoubleClicked;
	FOnClicked OnFilterMiddleButtonClicked;
};

/**
 * A single filtering preset in the preset list. Can be removed by clicking the remove button on it.
 */
class SFilterPreset : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnRequestRemove, const TSharedRef<SFilterPreset>& /*PresetToRemove*/);
	DECLARE_DELEGATE_OneParam(FOnRequestEnableOnly, const TSharedRef<SFilterPreset>& /*PresetToEnable*/);
	DECLARE_DELEGATE_OneParam(FOnRequestDelete, const TSharedRef<SFilterPreset>& /*PresetToDelete*/);
	DECLARE_DELEGATE_OneParam(FOnRequestSave, const TSharedRef<SFilterPreset>& /*PresetToSave*/);
	DECLARE_DELEGATE_OneParam(FOnHighlightPreset, const TSharedPtr<ITraceFilterPreset>& /*PresetToHighlight*/);
	DECLARE_DELEGATE(FOnRequestEnableAll);
	DECLARE_DELEGATE(FOnRequestDisableAll);
	DECLARE_DELEGATE(FOnRequestRemoveAll);	

	SLATE_BEGIN_ARGS(SFilterPreset) {}
		/** Filter preset this widget represents */
		SLATE_ARGUMENT(TSharedPtr<ITraceFilterPreset>, FilterPreset)

		/** Invoked when the preset toggled */
		SLATE_EVENT(FOnPresetChanged, OnPresetChanged)

		/** Invoked when a request to remove this preset originated from within this widget */
		SLATE_EVENT(FOnRequestRemove, OnRequestRemove)

		/** Invoked when a request to enable only this preset originated from within this widget */
		SLATE_EVENT(FOnRequestEnableOnly, OnRequestEnableOnly)

		/** Invoked when a request to delete this preset originated from within this widget */
		SLATE_EVENT(FOnRequestDelete, OnRequestDelete);

		/** Invoked when a request to save the current filtering state as this preset originated from within this widget */
		SLATE_EVENT(FOnRequestSave, OnRequestSave);

		/** Invoked when a request to highlight this preset originated from within this widget */
		SLATE_EVENT(FOnHighlightPreset, OnHighlightPreset)

		/** Invoked when a request to enable all presets originated from within this widget */
		SLATE_EVENT(FOnRequestEnableAll, OnRequestEnableAll)

		/** Invoked when a request to disable all presets originated from within this widget */
		SLATE_EVENT(FOnRequestDisableAll, OnRequestDisableAll)

		/** Invoked when a request to remove all presets originated from within this widget */
		SLATE_EVENT(FOnRequestRemoveAll, OnRequestRemoveAll)
	SLATE_END_ARGS()

	virtual ~SFilterPreset();

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

	/** Begin SCompoundWidget overrides */
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	/** End SCompoundWidget overrides */

	/** Sets the enabled state but doesn't call the callback that changes the channel's state */
	void MarkAsEnabled(bool InEnabled);

	/** Sets whether or not this preset is applied to the combined filtering state */
	void SetEnabled(bool InEnabled);

	/** Returns true if this preset contributes to the combined filtering state */
	bool IsEnabled() const;

	/** Get the preset object this widget represents */
	const TSharedPtr<ITraceFilterPreset>& GetFilterPreset() const;
private:
	/** Handler for when the preset checkbox is clicked */
	void PresetToggled(ECheckBoxState NewState);

	/** Handler for when the preset checkbox is clicked and a control key is pressed */
	FReply FilterCtrlClicked();

	/** Handler for when the preset checkbox is clicked and an alt key is pressed */
	FReply FilterAltClicked();

	/** Handler for when the preset checkbox is double clicked */
	FReply FilterDoubleClicked();

	/** Handler for when the preset checkbox is middle button clicked */
	FReply FilterMiddleButtonClicked();

	FText GetToolTipText() const;

	/** Handler to create a right click menu */
	TSharedRef<SWidget> GetRightClickMenuContent();

	/** Removes this preset from the presets list */
	void RemovePreset();

	/** Enables only this preset from the presets list */
	void EnableOnly();

	/** Enables all presets in the list */
	void EnableAllPresets();

	/** Disables all active presets in the list */
	void DisableAllPresets();

	/** Removes all presets in the list */
	void RemoveAllPresets();

	/** Saves the current filtering state as this preset */
	void SavePreset();

	/** Deletes this preset */
	void DeletePreset();

	/** Handler to determine the "checked" state of the preset checkbox */
	ECheckBoxState IsChecked() const;

	/** Handler to determine the color of the checkbox when it is checked */
	FSlateColor GetPresetForegroundColor() const;

	/** Handler to determine the padding of the checkbox text when it is pressed */
	FMargin GetNameLabelPadding() const;

	/** Handler to determine the color of the checkbox text when it is hovered */
	FSlateColor GetNameLabelColorAndOpacity() const;

	/** Returns the display name for this filter */
	FText GetPresetName() const;

private:
	TSharedPtr<ITraceFilterPreset> FilterPreset;

	FOnPresetChanged OnPresetChanged;
	FOnRequestRemove OnRequestRemove;
	FOnRequestDelete OnRequestDelete;
	FOnRequestSave OnRequestSave;
	FOnHighlightPreset OnHighlightPreset;
	FOnRequestEnableOnly OnRequestEnableOnly;
	FOnRequestEnableAll OnRequestEnableAll;
	FOnRequestDisableAll OnRequestDisableAll;
	FOnRequestDisableAll OnRequestRemoveAll;	

	/** True when this preset should be applied to filtering state */
	bool bEnabled;

	/** The button to toggle the preset on or off */
	TSharedPtr<SFilterPresetCheckBox> ToggleButtonPtr;

	/** The color of the checkbox for this preset */
	FLinearColor FilterColor;

	/** True when this preset its filtering state is currently set to being highlighted */
	bool bHighlighted;
};

} // namespace UE::TraceTools