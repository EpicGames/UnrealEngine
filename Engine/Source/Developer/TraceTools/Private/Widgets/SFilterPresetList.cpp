// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFilterPresetList.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Layout/WidgetPath.h"
#include "Misc/ConfigCacheIni.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Input/SEditableTextBox.h"

// TraceTools
#include "Models/TraceFilterPresets.h"
#include "TraceToolsModule.h"
#include "Services/SessionTraceControllerFilterService.h"

#if WITH_EDITOR
#include "SSettingsEditorCheckoutNotice.h"
#endif

#define LOCTEXT_NAMESPACE "InsightsFilterList"

namespace UE::TraceTools
{

void SFilterPresetList::Construct( const FArguments& InArgs, TSharedPtr<ISessionTraceFilterService> InSessionFilterService)
{
	OnPresetChanged = InArgs._OnPresetChanged;
	OnSavePreset = InArgs._OnSavePreset;
	OnHighlightPreset = InArgs._OnHighlightPreset;

	SessionFilterService = InSessionFilterService;

	FilterBox = SNew(SWrapBox)
				.UseAllottedSize(true);

	LoadSettings(FTraceToolsModule::TraceFiltersIni);
	
	PresetContainer = GetMutableDefault<ULocalTraceFilterPresetContainer>();
	SharedPresetContainer = GetMutableDefault<USharedTraceFilterPresetContainer>();

	RefreshFilterPresets();

	ChildSlot
	[
		FilterBox.ToSharedRef()
	];

}

SFilterPresetList::~SFilterPresetList()
{
	SaveSettings(FTraceToolsModule::TraceFiltersIni);
}

void SFilterPresetList::RefreshPresetEnabledState()
{
	TArray<FString> Names;

	for (TSharedRef<SFilterPreset> PresetWidget : Presets)
	{
		Names.Empty();
		bool bPresetIsEnabled = true;
		PresetWidget->GetFilterPreset()->GetAllowlistedNames(Names);
		for (const FString& Name : Names)
		{
			const FTraceObjectInfo* Object = SessionFilterService->GetObject(Name);
			if (Object == nullptr || !Object->bEnabled)
			{
				bPresetIsEnabled = false;
				break;
			}
		}

		PresetWidget->MarkAsEnabled(bPresetIsEnabled);
	}
}

void SFilterPresetList::RefreshFilterPresets()
{
	FSlateApplication::Get().DismissAllMenus();

	if (HasAnyPresets())
	{
		for (auto FilterIt = Presets.CreateConstIterator(); FilterIt; ++FilterIt)
		{
			const TSharedRef<SFilterPreset>& FilterToRemove = *FilterIt;
			CurrentActiveFilterNames.Add(FilterToRemove->GetFilterPreset()->GetName());
		}

		FilterBox->ClearChildren();
		Presets.Empty();
	}

	Presets.Empty();
	EngineFilterPresets.Empty();
	UserFilterPresets.Empty();
	SharedUserFilterPresets.Empty();
	AllFilterPresets.Empty();

	LoadEnginePresets();
	
	PresetContainer->GetUserPresets(UserFilterPresets);
	SharedPresetContainer->GetSharedUserPresets(SharedUserFilterPresets);

	AllFilterPresets.Append(EngineFilterPresets);
	AllFilterPresets.Append(UserFilterPresets);
	AllFilterPresets.Append(SharedUserFilterPresets);

	for (const TSharedPtr<ITraceFilterPreset>& Preset : AllFilterPresets)
	{
		if (CurrentActiveFilterNames.Contains(Preset->GetName()))
		{
			TSharedRef<SFilterPreset> Filter = AddFilterPreset(Preset);
			CurrentActiveFilterNames.Remove(Preset->GetName());
		}
	}

	RefreshPresetEnabledState();
}

FReply SFilterPresetList::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton )
	{
		FReply Reply = FReply::Handled().ReleaseMouseCapture();

		// Get the context menu content. If NULL, don't open a menu.
		TSharedPtr<SWidget> MenuContent = MakeFilterPresetsMenu();

		if ( MenuContent.IsValid() )
		{
			FVector2D SummonLocation = MouseEvent.GetScreenSpacePosition();
			FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
			FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuContent.ToSharedRef(), SummonLocation, FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
		}

		return Reply;
	}

	return FReply::Unhandled();
}

void SFilterPresetList::SaveSettings(const FString& IniFilename) const
{
	if (GConfig == nullptr)
	{
		return;
	}

	FStringView IniSectionName = TEXTVIEW("Trace.FilterPresetList");
	FStringView IniActivePresetsKey = TEXTVIEW("ActivePresets");

	FStringView Separator = TEXTVIEW(";");
	
	FString ActivePresetsString;

	for (const TSharedRef<SFilterPreset>& Preset : Presets)
	{
		ActivePresetsString.Append(Preset->GetFilterPreset()->GetName());
		ActivePresetsString.Append(Separator);
	}

	ActivePresetsString.RemoveFromEnd(Separator);

	GConfig->SetString(IniSectionName.GetData(), IniActivePresetsKey.GetData(), *ActivePresetsString, IniFilename);

	GConfig->Flush(false, IniFilename);
}

void SFilterPresetList::LoadSettings(const FString& IniFilename)
{
	FStringView IniSectionName = TEXTVIEW("Trace.FilterPresetList");
	FStringView IniActivePresetsKey = TEXT("ActivePresets");

	FString ActivePresetsString;
	FString EnabledPresetsString;
	GConfig->GetString(IniSectionName.GetData(), IniActivePresetsKey.GetData(), ActivePresetsString, IniFilename);

	FStringView Separator = TEXT(";");
	ActivePresetsString.ParseIntoArray(CurrentActiveFilterNames, Separator.GetData());
}

bool SFilterPresetList::HasAnyPresets() const
{
	return Presets.Num() > 0;
}

void SFilterPresetList::EnableAllPresets()
{
	for (auto FilterIt = Presets.CreateConstIterator(); FilterIt; ++FilterIt)
	{
		(*FilterIt)->SetEnabled(true);
	}
}

void SFilterPresetList::DisableAllPresets()
{
	for ( auto FilterIt = Presets.CreateConstIterator(); FilterIt; ++FilterIt )
	{
		(*FilterIt)->SetEnabled(false);
	}
}

void SFilterPresetList::RemoveAllPresets()
{
	if ( HasAnyPresets() )
	{
		DisableAllPresets();
		FilterBox->ClearChildren();
		Presets.Empty();
	}
}


void SFilterPresetList::GenerateEnginePresetsMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.BeginSection("TracePresetsEnginePresets", LOCTEXT("EnginePresetsMenuHeading", "Engine Presets"));
	{
		for (const TSharedPtr<ITraceFilterPreset>& EnginePreset : EngineFilterPresets)
		{
			InMenuBuilder.AddMenuEntry(
				EnginePreset->GetDisplayText(),
				EnginePreset->GetDisplayText(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SFilterPresetList::TogglePreset, EnginePreset),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(this, &SFilterPresetList::IsPresetEnabled, EnginePreset)),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}
	}
	InMenuBuilder.EndSection();  // TracePresetsEnginePresets
}

void SFilterPresetList::GenerateLocalUserPresetsMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.BeginSection("TracePresetsUserPresets", LOCTEXT("UserPresetsMenuHeading", "User Presets"));
	{
		for (const TSharedPtr<ITraceFilterPreset>& UserPreset : UserFilterPresets)
		{
			InMenuBuilder.AddSubMenu(
				UserPreset->GetDisplayText(),
				UserPreset->GetDisplayText(),
				FNewMenuDelegate::CreateLambda([this, UserPreset](FMenuBuilder& InSubMenuBuilder)
				{
					InSubMenuBuilder.BeginSection(NAME_None, LOCTEXT("UserPresetItemsHead", "User Preset(s)"));
					{
						GenerateCommonPresetEntries(InSubMenuBuilder, UserPreset);
					}
					InSubMenuBuilder.EndSection();
				}),
				FUIAction(
					FExecuteAction::CreateSP(this, &SFilterPresetList::TogglePreset, UserPreset),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(this, &SFilterPresetList::IsPresetEnabled, UserPreset)),
				NAME_None,
				EUserInterfaceActionType::ToggleButton,
				false, 
				FSlateIcon(),
				false
			);
		}
	}
	InMenuBuilder.EndSection(); // TracePresetsUserPresets
}

void SFilterPresetList::GenerateSharedUserPresetsMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.BeginSection("TracePresetsSharedUserPresets", LOCTEXT("SharedUserPresetsMenuHeading", "Shared User Presets"));
	{
		for (const TSharedPtr<ITraceFilterPreset>& UserPreset : SharedUserFilterPresets)
		{
			InMenuBuilder.AddSubMenu(
				UserPreset->GetDisplayText(),
				UserPreset->GetDisplayText(),
				FNewMenuDelegate::CreateLambda([this, UserPreset](FMenuBuilder& InSubMenuBuilder)
				{
#if WITH_EDITOR
					// SSettingsEditorCheckoutNotice is part of SharedWidgets module which depends on UnrealEd
					TSharedPtr<SSettingsEditorCheckoutNotice> CheckoutWidget;
					
					SAssignNew(CheckoutWidget, SSettingsEditorCheckoutNotice)
					.ConfigFilePath(GetDefault<USharedTraceFilterPresetContainer>()->GetDefaultConfigFilename())
					.Visibility_Lambda([]() -> EVisibility
					{
						return FFilterPresetHelpers::CanModifySharedPreset() ? EVisibility::Collapsed : EVisibility::Visible;
					});
#endif // WITH_EDITOR
					InSubMenuBuilder.BeginSection(NAME_None, LOCTEXT("SharedUserPresetItemHead", "Shared User Preset(s)"));
					{
#if WITH_EDITOR
						InSubMenuBuilder.AddWidget(CheckoutWidget.ToSharedRef(), FText::GetEmpty());
#endif // WITH_EDITOR
						GenerateCommonPresetEntries(InSubMenuBuilder, UserPreset);
					}
					InSubMenuBuilder.EndSection();
				}),
				FUIAction(
					FExecuteAction::CreateSP(this, &SFilterPresetList::TogglePreset, UserPreset),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(this, &SFilterPresetList::IsPresetEnabled, UserPreset)),
				NAME_None,
				EUserInterfaceActionType::ToggleButton,
				false, 
				FSlateIcon(),
				false
			);
		}
	}
	InMenuBuilder.EndSection(); // TracePresetsSharedUserPresets
}

void SFilterPresetList::GenerateCommonPresetEntries(FMenuBuilder& InSubMenuBuilder, const TSharedPtr<ITraceFilterPreset>& UserPreset)
{
	InSubMenuBuilder.AddWidget(
		SNew(SEditableTextBox)
		.Text_Lambda([UserPreset]() { return UserPreset->GetDisplayText(); })
		.OnTextCommitted_Lambda([UserPreset](const FText& InText, ETextCommit::Type InCommitType)
		{
			UserPreset->Rename(InText.ToString());
		})
		.OnVerifyTextChanged_Lambda([this, UserPreset](const FText& InNewText, FText& OutErrorMessage) -> bool
		{
			auto NameTestPredicate = [this, UserPreset, InNewText](TSharedPtr<ITraceFilterPreset> TestPreset)
			{
				return TestPreset != UserPreset && TestPreset->GetDisplayText().ToString() == InNewText.ToString();
			};

			if (UserFilterPresets.ContainsByPredicate(NameTestPredicate) || SharedUserFilterPresets.ContainsByPredicate(NameTestPredicate))
			{
				OutErrorMessage = LOCTEXT("DuplicatePresetNames", "This name is already in use");
				return false;
			}
			return true;
		})
		, FText::GetEmpty()
	);
	
	const FText DisplayText = UserPreset->IsLocal() ? LOCTEXT("MakeSharedPresetLabel", "Make Shared Preset") : LOCTEXT("MakeLocalPresetLabel", "Make Local Preset");
	const FText TooltipText = UserPreset->IsLocal() ? LOCTEXT("MakeSharedPresetToolTip", "Makes this preset a Shared User Preset (Config INI file has to be writable)") : LOCTEXT("MakeLocalPresetToolTip", "Makes this preset a Local Preset");

	InSubMenuBuilder.AddMenuEntry(DisplayText,
		TooltipText,
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([UserPreset, this]()
			{
				const bool bSuccess = UserPreset->IsLocal() ? UserPreset->MakeShared() : UserPreset->MakeLocal();
				if (bSuccess)
				{
					RefreshFilterPresets();
				}
			}),
			FCanExecuteAction::CreateLambda([]()
			{
				return FFilterPresetHelpers::CanModifySharedPreset();
			})
		)
	);

		InSubMenuBuilder.AddMenuEntry(LOCTEXT("PresetSaveLabel", "Save Preset"),
		LOCTEXT("PresetSaveToolTip", "Saves the current filtering state as this User Preset"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([UserPreset, this]()
			{
				if (UserPreset.IsValid())
				{
					OnSavePreset.ExecuteIfBound(UserPreset);
				}
			})
		)
	);

	InSubMenuBuilder.AddMenuEntry(LOCTEXT("PresetDeleteLabel", "Delete Preset"),
		LOCTEXT("PresetDeleteToolTip", "Deletes the User Preset"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([UserPreset, this]()
			{
				if ( UserPreset->Delete())
				{
					RefreshFilterPresets();
				}
			})
		)
	);
}

void SFilterPresetList::TogglePreset(const TSharedPtr<ITraceFilterPreset> InPreset)
{
	// Toggle off 
	for (TSharedRef<SFilterPreset> Filter : Presets)
	{
		if (Filter->GetFilterPreset() == InPreset)
		{
			RemoveFilterPresetAndUpdate(Filter);
			return;
		}
	}

	// Toggle on
	TSharedPtr<SFilterPreset> PresetWidget = AddFilterPreset(InPreset);
	PresetWidget->SetEnabled(true);
}

bool SFilterPresetList::IsPresetEnabled(const TSharedPtr<ITraceFilterPreset> InPreset) const 
{
	for (TSharedRef<SFilterPreset> Filter : Presets)
	{
		if (Filter->GetFilterPreset() == InPreset)
		{
			return true;
		}
	}

	return false;
}

TSharedRef<SWidget> SFilterPresetList::MakeFilterPresetsMenu()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/false, nullptr, nullptr, /*bCloseSelfOnly=*/true);

	MenuBuilder.BeginSection("TracePresetsResetPresets");
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("FilterListResetPresets", "Reset Presets"),
			LOCTEXT("FilterListResetToolTip", "Resets current presets selection"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SFilterPresetList::RemoveAllPresets))
		);
		
		MenuBuilder.AddMenuEntry(
			LOCTEXT("FilterListSavePresets", "Save as User Preset"),
			LOCTEXT("FilterListSaveToolTip", "Saves the currently filtering state as a new User Preset"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([this]() 
			{
				// Cache all user names to determine which preset is the newly created one
				TArray<FString> Names;
				Algo::Transform(UserFilterPresets, Names, [](TSharedPtr<ITraceFilterPreset> Preset)
				{
					return Preset->GetName();
				});

				OnSavePreset.ExecuteIfBound(nullptr); 
				RefreshFilterPresets();

				for (const TSharedPtr<ITraceFilterPreset>& Preset : UserFilterPresets)
				{
					if (!Names.Contains(Preset->GetName()))
					{
						// This is the newly created preset, enable it for UX
						TSharedRef<SFilterPreset> Widget = AddFilterPreset(Preset);
						EnableOnlyThisPreset(Widget);
						break;
					}
				}
			}))
		);
	}
	MenuBuilder.EndSection(); // TracePresetsResetPresets

	GenerateEnginePresetsMenu(MenuBuilder);
	GenerateLocalUserPresetsMenu(MenuBuilder);
	GenerateSharedUserPresetsMenu(MenuBuilder);

	return MenuBuilder.MakeWidget();
}

TSharedRef<SFilterPreset> SFilterPresetList::AddFilterPreset(const TSharedPtr<ITraceFilterPreset> FilterPreset)
{
	TSharedRef<SFilterPreset> NewFilter = SNew(SFilterPreset)
		.FilterPreset(FilterPreset)
		.OnPresetChanged(OnPresetChanged)
		.OnRequestRemove(this, &SFilterPresetList::RemoveFilterPresetAndUpdate)
		.OnRequestEnableAll(this, &SFilterPresetList::EnableAllPresets)
		.OnRequestEnableOnly(this, &SFilterPresetList::EnableOnlyThisPreset)
		.OnRequestDisableAll(this, &SFilterPresetList::DisableAllPresets)
		.OnRequestRemoveAll(this, &SFilterPresetList::RemoveAllPresets)
		.OnRequestDelete(this, & SFilterPresetList::DeletePreset)
		.OnRequestSave(this, &SFilterPresetList::SavePreset)
		.OnHighlightPreset(OnHighlightPreset);

	Presets.Add(NewFilter);

	// Add the widget as a child
	FilterBox->AddSlot()
	.Padding(3, 3)
	[
		NewFilter
	];

	return NewFilter;
}

void SFilterPresetList::EnableOnlyThisPreset(const TSharedRef<SFilterPreset>& PresetWidgetToEnable)
{
	for (TSharedRef<SFilterPreset>& PresetWidget : Presets )
	{
		const bool bEnable = (PresetWidget == PresetWidgetToEnable);
		PresetWidget->SetEnabled(bEnable);
	}
}

void SFilterPresetList::RemoveFilterPresetAndUpdate(const TSharedRef<SFilterPreset>& PresetToRemove)
{
	FilterBox->RemoveSlot(PresetToRemove);
	Presets.RemoveSingleSwap(PresetToRemove);
	PresetToRemove->SetEnabled(false);
}

void SFilterPresetList::DeletePreset(const TSharedRef<SFilterPreset>& PresetToDelete)
{
	if (PresetToDelete->GetFilterPreset()->Delete())
	{
		RefreshFilterPresets();
	}
}

void SFilterPresetList::SavePreset(const TSharedRef<SFilterPreset>& PresetToSave)
{
	OnSavePreset.ExecuteIfBound(PresetToSave->GetFilterPreset());
	EnableOnlyThisPreset(PresetToSave);
}

void SFilterPresetList::GetAllEnabledPresets(TArray<TSharedPtr<ITraceFilterPreset>>& OutPresets) const
{
	for (const TSharedRef<SFilterPreset>& Filter : Presets)
	{
		if (Filter->IsEnabled())
		{
			OutPresets.Add(Filter->GetFilterPreset());
		}
	}
}

TSharedRef<SWidget> SFilterPresetList::ExternalMakeFilterPresetsMenu()
{
	return MakeFilterPresetsMenu();
}

void SFilterPresetList::LoadEnginePresets()
{
	if (SessionFilterService->HasSettings())
	{
		const TArray<FTraceStatus::FChannelPreset>& RuntimePresets = SessionFilterService->GetSettings().ChannelPresets;
		
		for (const FTraceStatus::FChannelPreset& Preset : RuntimePresets)
		{
			if (!Preset.bIsReadOnly)
			{
				TArray<FString> AllowListedNames;
				Preset.ChannelList.ParseIntoArray(AllowListedNames, TEXT(","));
				EngineFilterPresets.Add(MakeShared<FEngineFilterPreset>(Preset.Name, AllowListedNames));
			}
		}
	}
}

} // namespace UE::TraceTools

#undef LOCTEXT_NAMESPACE
