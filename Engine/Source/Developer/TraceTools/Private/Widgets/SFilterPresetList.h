// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SFilterPreset.h"

class FMenuBuilder;
class SWrapBox;

class ULocalTraceFilterPresetContainer;
class USharedTraceFilterPresetContainer;

namespace UE::TraceTools
{

class ISessionTraceFilterService;
class SFilterPreset;

/**
 * A list of presets currently applied to insights filtering window
 */
class SFilterPresetList : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnSavePreset, const TSharedPtr<ITraceFilterPreset>& /*PresetToSave, nullptr means create a new one */);
	DECLARE_DELEGATE_OneParam(FOnHighlightPreset, const TSharedPtr<ITraceFilterPreset>& /*PresetToHighlight, nullptr means stop highlighting */);

	SLATE_BEGIN_ARGS( SFilterPresetList ){}
		/** Delegate for when the selected presets have changed */
		SLATE_EVENT(FOnPresetChanged, OnPresetChanged)
		/** Delegate for when a preset is to be saved, given the current filtering state */
		SLATE_EVENT(FOnSavePreset, OnSavePreset)
		/** Delegate for when a specific preset is to be highlighted outside of this widget */
		SLATE_EVENT(FOnHighlightPreset, OnHighlightPreset)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */		
	void Construct( const FArguments& InArgs, TSharedPtr<ISessionTraceFilterService> InSessionFilterService);

	virtual ~SFilterPresetList();

	/** Begin SCompoundWidget overrides */
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	/** End SCompoundWidget overrides */

	/** Returns all filter presets currently set as enabled within this widget */
	void GetAllEnabledPresets(TArray<TSharedPtr<ITraceFilterPreset>>& OutPresets) const;

	/** Returns the context menu widget generated by MakeFilterPresetsMenu */
	TSharedRef<SWidget> ExternalMakeFilterPresetsMenu();	

	/** Sets channels as enabled/disabled based on the current object list */
	void RefreshPresetEnabledState();

	/** Refreshes the filter preset objects and child widgets */
	void RefreshFilterPresets();

private:
	/** Returns true if any Presets are applied */
	bool HasAnyPresets() const;

	/** Enables all Presets */
	void EnableAllPresets();

	/** Disables any applied Presets */
	void DisableAllPresets();

	/** Removes all Presets in the list */
	void RemoveAllPresets();

	/** Adds a child widget for the specified filter preset object */
	TSharedRef<SFilterPreset> AddFilterPreset(const TSharedPtr<ITraceFilterPreset> FilterPreset);

	/** Removes the child widget, and calls OnPresetsChanged to ensure the state is updated outside of this widget */
	void RemoveFilterPresetAndUpdate(const TSharedRef<SFilterPreset>& PresetToRemove);

	/** Handler for when the enable only this button was clicked on a single preset */
	void EnableOnlyThisPreset(const TSharedRef<SFilterPreset>& PresetWidgetToEnable);

	void TogglePreset(const TSharedPtr<ITraceFilterPreset> InPreset);
	bool IsPresetEnabled(const TSharedPtr<ITraceFilterPreset> InPreset) const;

	/** Saves any settings to config that should be persistent between editor sessions */
	void SaveSettings(const FString& IniFilename) const;

	/** Loads any settings to config that should be persistent between editor sessions */
	void LoadSettings(const FString& IniFilename);

	TSharedRef<SWidget> MakeFilterPresetsMenu();
	void GenerateEnginePresetsMenu(FMenuBuilder& InMenuBuilder);
	void GenerateLocalUserPresetsMenu(FMenuBuilder& InMenuBuilder);
	void GenerateCommonPresetEntries(FMenuBuilder& InSubMenuBuilder, const TSharedPtr<ITraceFilterPreset>& UserPreset);
	void GenerateSharedUserPresetsMenu(FMenuBuilder& InMenuBuilder);

	/** Delete the preset object contained by the widget */
	void DeletePreset(const TSharedRef<SFilterPreset>& PresetToDelete);
	/** Save the preset object contained by the widget */
	void SavePreset(const TSharedRef<SFilterPreset>& PresetToSave);	

	/** Engine presets have a custom loading so they can be shared with the console commands from FTraceAuxiliary. */
	void LoadEnginePresets();

private:
	/** The horizontal box which contains all the Presets */
	TSharedPtr<SWrapBox> FilterBox;

	/** All SFilterPreset in the list */
	TArray<TSharedRef<SFilterPreset>> Presets;
	
	/** All Filter Preset objects */
	TArray<TSharedPtr<ITraceFilterPreset>> AllFilterPresets;

	/** All User Filter Preset objects */
	TArray<TSharedPtr<ITraceFilterPreset>> UserFilterPresets;
	/** All Shared User Filter Preset objects */
	TArray<TSharedPtr<ITraceFilterPreset>> SharedUserFilterPresets;
	/** All Engine Preset objects */
	TArray<TSharedPtr<ITraceFilterPreset>> EngineFilterPresets;

	/** Weak pointers to filter preset containers */
	TWeakObjectPtr<ULocalTraceFilterPresetContainer> PresetContainer;
	TWeakObjectPtr<USharedTraceFilterPresetContainer> SharedPresetContainer;

	/** Delegate for when Presets have changed */
	FOnPresetChanged OnPresetChanged;

	/** Delegate for when a Preset is to be saved */
	FOnSavePreset OnSavePreset;

	/** Delegate for when a Preset is to be highlighted */
	FOnHighlightPreset OnHighlightPreset;

	/** Cached array of currently active filter names, used for saving/restoring state during refreshes */
	TArray<FString> CurrentActiveFilterNames;

	/** A pointer to the SessionFilterService to autodetect enabled presets. */
	TSharedPtr<ISessionTraceFilterService> SessionFilterService;
};

} // namespace UE::TraceTools