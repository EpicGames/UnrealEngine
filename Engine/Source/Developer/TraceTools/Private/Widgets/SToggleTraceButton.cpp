// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceTools/Widgets/SToggleTraceButton.h"

#include "Framework/Application/SlateApplication.h"
#include "Internationalization/Text.h"
#include "SlateOptMacros.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

//TraceTools
#include "Services/SessionTraceControllerFilterService.h"
#include "TraceToolsStyle.h"

#define LOCTEXT_NAMESPACE "SToggleTraceButton"

namespace UE::TraceTools
{

SLATE_IMPLEMENT_WIDGET(SToggleTraceButton)

void SToggleTraceButton::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "IsTraceRunning", IsTraceRunningAttribute, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "DynamicToolTipText", DynamicToolTipTextAttribute, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "LabelText", LabelTextAttribute, EInvalidateWidgetReason::Paint);
}

SToggleTraceButton::SToggleTraceButton()
	: IsTraceRunningAttribute(*this)
	, DynamicToolTipTextAttribute(*this)
	, LabelTextAttribute(*this)
{
}

SToggleTraceButton::~SToggleTraceButton()
{
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SToggleTraceButton::Construct(const FArguments& InArgs)
{
	OnToggleTraceRequested = InArgs._OnToggleTraceRequested;
	IsTraceRunningAttribute.Assign(*this, InArgs._IsTraceRunning);
	DynamicToolTipTextAttribute.Assign(*this, InArgs._DynamicToolTipText);
	LabelTextAttribute.Assign(*this, InArgs._LabelText);
	ButtonSize = InArgs._ButtonSize;
	bShowStopButton = InArgs._ShowStopButton;

	ChildSlot
	[
		SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.ContentPadding(FMargin(0.0f, 0.0f, 0.0f, 3.0f))
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Bottom)
		.ToolTipText(this, &SToggleTraceButton::GetRecordingButtonTooltipText)
		.OnClicked_Lambda([this]() { this->ToggleTrace_OnClicked(); return FReply::Handled(); })
		.OnHovered_Lambda([this]() { this->bIsTraceRecordButtonHovered = true; })
		.OnUnhovered_Lambda([this]() { this->bIsTraceRecordButtonHovered = false; })
		.Content()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			[
				SNew(SOverlay)

				+ SOverlay::Slot()
				[
					SNew(SImage)
					.ColorAndOpacity(this, &SToggleTraceButton::GetRecordingButtonColor)
					.Image(GetToggleTraceCenterBrush())
					.Visibility(this, &SToggleTraceButton::GetStartTraceIconVisibility)
				]

				+ SOverlay::Slot()
				[
					SNew(SImage)
					.ColorAndOpacity(this, &SToggleTraceButton::GetRecordingButtonOutlineColor)
					.Image(GetToggleTraceOutlineBrush())
					.Visibility(this, &SToggleTraceButton::GetStartTraceIconVisibility)
				]

				+ SOverlay::Slot()
				[
					SNew(SImage)
					.Image(GetToggleTraceStopBrush())
					.Visibility(this, &SToggleTraceButton::GetStopTraceIconVisibility)
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(4.0f, 2.0f, 0.0f, 0.0f))
			[
				SNew(STextBlock)
				.Text(this, &SToggleTraceButton::GetRecordingButtonLabelText)
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

EVisibility SToggleTraceButton::GetStartTraceIconVisibility() const
{
	if (GetStopTraceIconVisibility() == EVisibility::Hidden)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}

EVisibility SToggleTraceButton::GetStopTraceIconVisibility() const
{
	if (bShowStopButton && bIsTraceRecordButtonHovered && IsTraceRunningAttribute.Get())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}

FSlateColor SToggleTraceButton::GetRecordingButtonColor() const
{
	if (!IsTraceRunningAttribute.Get())
	{
		return FStyleColors::White;
	}

	return FStyleColors::Error;
}

FSlateColor SToggleTraceButton::GetRecordingButtonOutlineColor() const
{
	if (!IsTraceRunningAttribute.Get())
	{
		ConnectionStartTime = FSlateApplication::Get().GetCurrentTime();
		return FLinearColor::White.CopyWithNewOpacity(0.5f);
	}

	double ElapsedTime = FSlateApplication::Get().GetCurrentTime() - ConnectionStartTime;
	return FStyleColors::Error.GetColor(FWidgetStyle()).CopyWithNewOpacity(0.5f + 0.5f * FMath::MakePulsatingValue(ElapsedTime, 0.5f));
}

FText SToggleTraceButton::GetRecordingButtonTooltipText() const
{
	FText ToolTipText = DynamicToolTipTextAttribute.Get();
	if (ToolTipText.IsEmpty())
	{
		if (!IsTraceRunningAttribute.Get())
		{
			return LOCTEXT("StartTracing", "Start tracing. The trace destination is set from the menu.");
		}

		return LOCTEXT("StopTracing", "Stop Tracing.");
	}

	return ToolTipText;
}

FText SToggleTraceButton::GetRecordingButtonLabelText() const
{
	return LabelTextAttribute.Get();
}

void SToggleTraceButton::ToggleTrace_OnClicked() const
{
	OnToggleTraceRequested.ExecuteIfBound();
}

const FSlateBrush* SToggleTraceButton::GetToggleTraceCenterBrush() const
{
	switch (ButtonSize)
	{
	case EButtonSize::StatusBar:
		return FTraceToolsStyle::Get().GetBrush("ToggleTraceButton.RecordTraceCenter.StatusBar");
	case EButtonSize::SlimToolbar:
		return FTraceToolsStyle::Get().GetBrush("ToggleTraceButton.RecordTraceCenter.SlimToolbar");
	}

	return nullptr;
}

const FSlateBrush* SToggleTraceButton::GetToggleTraceOutlineBrush() const
{
	switch (ButtonSize)
	{
	case EButtonSize::StatusBar:
		return FTraceToolsStyle::Get().GetBrush("ToggleTraceButton.RecordTraceOutline.StatusBar");
	case EButtonSize::SlimToolbar:
		return FTraceToolsStyle::Get().GetBrush("ToggleTraceButton.RecordTraceOutline.SlimToolbar");
	}

	return nullptr;
}

const FSlateBrush* SToggleTraceButton::GetToggleTraceStopBrush() const
{
	switch (ButtonSize)
	{
	case EButtonSize::StatusBar:
		return FTraceToolsStyle::Get().GetBrush("ToggleTraceButton.TraceStop.StatusBar");
	case EButtonSize::SlimToolbar:
		return FTraceToolsStyle::Get().GetBrush("ToggleTraceButton.TraceStop.SlimToolbar");
	}
	
	return nullptr;
}

} // namespace UE::TraceTools

#undef LOCTEXT_NAMESPACE