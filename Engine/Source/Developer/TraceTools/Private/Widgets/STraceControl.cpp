// Copyright Epic Games, Inc. All Rights Reserved.

#include "STraceControl.h"

#include "Framework/Commands/UICommandList.h"
#include "ISessionInstanceInfo.h"
#include "ISessionManager.h"
#include "ISessionServicesModule.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "STraceDataFilterWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/STraceControlToolbar.h"

// Trace Tools
#include "Services/SessionTraceControllerFilterService.h"

#define LOCTEXT_NAMESPACE "UE::TraceTools::STraceControl"

namespace UE::TraceTools
{

STraceControl::STraceControl()
{
}

STraceControl::~STraceControl()
{
	if (bAutoDetectSelectedSession && SessionManager.IsValid())
	{
		SessionManager->OnInstanceSelectionChanged().RemoveAll(this);
	}
}

void STraceControl::Construct(const FArguments& InArgs, TSharedPtr<ITraceController> InTraceController)
{
	TraceController = InTraceController;
	bAutoDetectSelectedSession = InArgs._AutoDetectSelectedSession;

	UICommandList = MakeShareable(new FUICommandList);

	SessionFilterService = MakeShareable(new FSessionTraceControllerFilterService(InTraceController));

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.Padding(0.0f, 4.0f, 0.0f, 0.0f)
		.AutoHeight()
		[
			SAssignNew(Toolbar, STraceControlToolbar, UICommandList.ToSharedRef(), InTraceController)
		]

		+ SVerticalBox::Slot()
		.Padding(FMargin(0.0f, 0.0f, 0.0f, 2.0f))
		[
			SAssignNew(TraceDataFilterWidget, STraceDataFilterWidget, InTraceController, SessionFilterService)
		]
	];

	if (bAutoDetectSelectedSession)
	{
		ISessionServicesModule& SessionServicesModule = FModuleManager::LoadModuleChecked<ISessionServicesModule>("SessionServices");
		SessionManager = SessionServicesModule.GetSessionManager();

		if (SessionManager.IsValid())
		{
			//Note that we should not add ourselves as shared pointer to session manager,
			//that will create a circular dependency
			SessionManager->OnInstanceSelectionChanged().AddRaw(this, &STraceControl::OnInstanceSelectionChanged);
		}

		TraceDataFilterWidget->SetWarningBannerText(LOCTEXT("NoSessionSelectedWarning", "Please select an active instance from the Session Browser."));

		const TArray<TSharedPtr<ISessionInstanceInfo>> CurrentSelectedSessions = SessionManager->GetSelectedInstances();
		for (TSharedPtr<ISessionInstanceInfo> Instance : CurrentSelectedSessions)
		{
			OnInstanceSelectionChanged(Instance, true);
		}
	}
}

void STraceControl::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	AccumulatedTime += InDeltaTime;

	constexpr double UpdateTime = 1.0f;

	if (InstanceId.IsValid() && AccumulatedTime > UpdateTime)
	{
		TraceController->WithInstance(InstanceId, [](const FTraceStatus& Status, ITraceControllerCommands& Commands)
		{
			Commands.SendStatusUpdateRequest();
			Commands.SendChannelUpdateRequest();
		});

		if (!SessionFilterService->HasSettings())
		{
			TraceController->WithInstance(InstanceId, [](const FTraceStatus& Status, ITraceControllerCommands& Commands)
			{
				Commands.SendSettingsUpdateRequest();
			});
		}

		AccumulatedTime = 0.0f;
	}
}

void STraceControl::SetInstanceId(const FGuid& Id)
{
	InstanceId = Id;
	Toolbar->SetInstanceId(Id);
	SessionFilterService->SetInstanceId(Id);

	if (InstanceId.IsValid())
	{
		TraceController->SendDiscoveryRequest(FGuid(), Id);

		TraceController->WithInstance(InstanceId, [](const FTraceStatus& Status, ITraceControllerCommands& Commands)
		{
			Commands.SendStatusUpdateRequest();
			Commands.SendChannelUpdateRequest();
		});
	}

#if !WITH_EDITOR
	if (Id == FApp::GetInstanceId())
	{
		TraceDataFilterWidget->SetWarningBannerText(LOCTEXT("SessionCannotBeControlledWarning", "Unreal Session Frontend cannot be controlled by this tool. Please select another active instance from the Session Browser."));
	}
	else
	{
		if (bAutoDetectSelectedSession)
		{
			TraceDataFilterWidget->SetWarningBannerText(LOCTEXT("NoSessionSelectedWarning", "Please select an active instance from the Session Browser."));
		}
		else
		{
			TraceDataFilterWidget->SetWarningBannerText(FText::GetEmpty());
		}
	}
#endif
}

void STraceControl::OnInstanceSelectionChanged(const TSharedPtr<ISessionInstanceInfo>& InstanceInfo, bool bSelected)
{
	if (!InstanceInfo.IsValid())
	{
		return;
	}

	if (bSelected)
	{
		SelectedSessionsIds.Add(InstanceInfo->GetInstanceId());
	}
	else
	{
		SelectedSessionsIds.Remove(InstanceInfo->GetInstanceId());
	}

	if (SelectedSessionsIds.Num() == 1)
	{
		SetInstanceId(*SelectedSessionsIds.begin());
	}
	else
	{
		InstanceId.Invalidate();
		SetInstanceId(InstanceId);
	}
}

} // namespace UE::TraceTools

#undef LOCTEXT_NAMESPACE // UE::TraceTools::STraceControl