// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ITraceController.h"
#include "Widgets/SCompoundWidget.h"

class ISessionInstanceInfo;
class ISessionManager;
class FUICommandList;

namespace UE::TraceTools
{

class ISessionTraceFilterService;
class STraceControlToolbar;
class STraceDataFilterWidget;

class STraceControl : public SCompoundWidget
{
public:
	/** Default constructor. */
	STraceControl();

	/** Virtual destructor. */
	virtual ~STraceControl();

	SLATE_BEGIN_ARGS(STraceControl) 
		: _AutoDetectSelectedSession(false)
	{}

	/* Specifies if the widget should autodetect the selected session in Session Browser. If false, the SessionId will need to be set manually. */
	SLATE_ARGUMENT(bool, AutoDetectSelectedSession)

	SLATE_END_ARGS()

	/** Constructs this widget. */
	void Construct(const FArguments& InArgs, TSharedPtr<ITraceController> InTraceController);

	/** Sets the InstanceId to control. Supports invalid guid value for disabled state. */
	void SetInstanceId(const FGuid& Id);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

protected:
	void OnInstanceSelectionChanged(const TSharedPtr<class ISessionInstanceInfo>&, bool);

private:
	TSharedPtr<ITraceController> TraceController;

	TSharedPtr<FUICommandList> UICommandList;

	/** The InstanceId to control. */
	FGuid InstanceId;

	TSharedPtr<STraceControlToolbar> Toolbar;

	TSharedPtr<STraceDataFilterWidget> TraceDataFilterWidget;

	/** Session manager used for selecting sessions */
	TSharedPtr<ISessionManager> SessionManager;

	TSharedPtr<ISessionTraceFilterService> SessionFilterService;

	TSet<FGuid> SelectedSessionsIds;

	bool bAutoDetectSelectedSession;

	double AccumulatedTime = 0.0f;
};

} // namespace UE::TraceTools
