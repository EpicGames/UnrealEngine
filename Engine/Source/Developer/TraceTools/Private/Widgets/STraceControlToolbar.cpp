// Copyright Epic Games, Inc. All Rights Reserved.

#include "STraceControlToolbar.h"

#include "ITraceController.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SlateOptMacros.h"
#include "SocketSubsystem.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"

//TraceTools
#include "Models/TraceControlCommands.h"
#include "TraceTools/Widgets/SToggleTraceButton.h"

#define LOCTEXT_NAMESPACE "STraceControlToolbar"

namespace UE::TraceTools
{

STraceControlToolbar::STraceControlToolbar()
{
}

STraceControlToolbar::~STraceControlToolbar()
{
	TraceController->OnStatusReceived().RemoveAll(this);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void STraceControlToolbar::Construct(const FArguments& InArgs, const TSharedRef<FUICommandList>& CommandList, TSharedPtr<ITraceController> InTraceController)
{
	InitializeSettings();

	FTraceControlCommands::Register();
	TraceController = InTraceController;

	OnStatusReceivedDelegate = TraceController->OnStatusReceived().AddRaw(this, &STraceControlToolbar::OnTraceStatusUpdated);

	BindCommands(CommandList);

	// create the toolbar
	FSlimHorizontalToolBarBuilder Toolbar(CommandList, FMultiBoxCustomization::None);
	{
		Toolbar.SetStyle(&FTraceToolsStyle::Get(), "TraceControlToolbar");
		Toolbar.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &STraceControlToolbar::BuildTraceTargetMenu, CommandList),
			TAttribute<FText>::CreateSP(this, &STraceControlToolbar::GetTraceTargetLabelText),
			TAttribute<FText>::CreateSP(this, &STraceControlToolbar::GetTraceTargetTooltipText),
			TAttribute<FSlateIcon>::CreateSP(this, &STraceControlToolbar::GetTraceTargetIcon),
			false);

		Toolbar.AddSeparator();

		Toolbar.SetLabelVisibility(EVisibility::Collapsed);

		TSharedRef<SWidget> ToggleTraceWidget = 
			SNew(SHorizontalBox)
			
			+ SHorizontalBox::Slot()
			.Padding(0.0f, 2.0f, 0.0f, 0.0f)
			[
				SNew(SToggleTraceButton)
				.OnToggleTraceRequested(this, &STraceControlToolbar::ToggleTrace_Execute)
				.IsTraceRunning_Lambda([this]() {return bIsTracing; })
				.IsEnabled(this, &STraceControlToolbar::ToggleTrace_CanExecute)
				.ButtonSize(SToggleTraceButton::EButtonSize::SlimToolbar)
			];


		Toolbar.AddToolBarWidget(ToggleTraceWidget);

		Toolbar.AddToolBarButton(FTraceControlCommands::Get().TraceSnapshot);

		Toolbar.AddSeparator();

		TSharedRef<SWidget> PauseResumeWidget =
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ContentPadding(FMargin(0.0f, 0.0f, 0.0f, 3.0f))
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Bottom)
				.ToolTipText(this, &STraceControlToolbar::TogglePauseResume_GetTooltip)
				.OnClicked(this, &STraceControlToolbar::TogglePauseResume_OnClicked)
				.IsEnabled(this, &STraceControlToolbar::TogglePauseResume_CanExecute)
				.Content()
				[
					SNew(SImage)
					.Image(this, &STraceControlToolbar::GetPauseResumeBrush)
					.ColorAndOpacity(FStyleColors::Foreground)
				]
			];

		Toolbar.AddWidget(PauseResumeWidget);

		Toolbar.AddSeparator();

		Toolbar.AddToolBarButton(FTraceControlCommands::Get().TraceBookmark);
		Toolbar.AddToolBarButton(FTraceControlCommands::Get().TraceScreenshot);

		Toolbar.AddSeparator();

		Toolbar.AddToolBarButton(FTraceControlCommands::Get().ToggleStatNamedEvents);
	}

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Padding(0.0f)
		[
			Toolbar.MakeWidget()
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedRef<SWidget> STraceControlToolbar::BuildTraceTargetMenu(const TSharedRef<FUICommandList> CommandList)
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, CommandList);

	MenuBuilder.SetSearchable(false);

	MenuBuilder.AddMenuEntry(FTraceControlCommands::Get().SetTraceTargetServer);
	MenuBuilder.AddMenuEntry(FTraceControlCommands::Get().SetTraceTargetFile);

	return MenuBuilder.MakeWidget();
}

void STraceControlToolbar::BindCommands(const TSharedRef<FUICommandList>& CommandList)
{
	CommandList->MapAction(FTraceControlCommands::Get().SetTraceTargetServer, 
						   FExecuteAction::CreateSP(this, &STraceControlToolbar::SetTraceTarget_Execute, ETraceTarget::Server),
						   FCanExecuteAction::CreateSP(this, &STraceControlToolbar::SetTraceTarget_CanExecute));

	CommandList->MapAction(FTraceControlCommands::Get().SetTraceTargetFile, 
						   FExecuteAction::CreateSP(this, &STraceControlToolbar::SetTraceTarget_Execute, ETraceTarget::File),
						   FCanExecuteAction::CreateSP(this, &STraceControlToolbar::SetTraceTarget_CanExecute));

	CommandList->MapAction(FTraceControlCommands::Get().TraceSnapshot, 
						   FExecuteAction::CreateSP(this, &STraceControlToolbar::TraceSnapshot_Execute),
						   FCanExecuteAction::CreateSP(this, &STraceControlToolbar::TraceSnapshot_CanExecute));

	CommandList->MapAction(FTraceControlCommands::Get().TraceBookmark, 
						   FExecuteAction::CreateSP(this, &STraceControlToolbar::TraceBookmark_Execute),
						   FCanExecuteAction::CreateSP(this, &STraceControlToolbar::TraceBookmark_CanExecute));

	CommandList->MapAction(FTraceControlCommands::Get().TraceScreenshot, 
						   FExecuteAction::CreateSP(this, &STraceControlToolbar::TraceScreenshot_Execute),
						   FCanExecuteAction::CreateSP(this, &STraceControlToolbar::TraceScreenshot_CanExecute));

	CommandList->MapAction(FTraceControlCommands::Get().ToggleStatNamedEvents, 
						   FExecuteAction::CreateSP(this, &STraceControlToolbar::ToggleStatNamedEvents_Execute),
						   FCanExecuteAction::CreateSP(this, &STraceControlToolbar::ToggleStatNamedEvents_CanExecute),
						   FIsActionChecked::CreateSP(this, &STraceControlToolbar::ToggleStatNamedEvents_IsChecked));
}

void STraceControlToolbar::InitializeSettings()
{
	TSharedPtr<FInternetAddr> RecorderAddr;
	if (ISocketSubsystem* Sockets = ISocketSubsystem::Get())
	{
		bool bCanBindAll = false;
		RecorderAddr = Sockets->GetLocalHostAddr(*GLog, bCanBindAll);
	}

	if (RecorderAddr.IsValid())
	{
		TraceHostAddr = RecorderAddr->ToString(false);
	}
	else
	{
		TraceHostAddr = TEXT("127.0.0.1");
	}
}

bool STraceControlToolbar::SetTraceTarget_CanExecute() const
{
	return IsInstanceAvailable() && !bIsTracing;
}

void STraceControlToolbar::SetTraceTarget_Execute(ETraceTarget InTraceTarget)
{
	TraceTarget = InTraceTarget;
}

bool STraceControlToolbar::IsInstanceAvailable() const
{
	return InstanceId.IsValid() && TraceController->HasAvailableInstance(InstanceId) && bIsTracingAvailable;
}

bool STraceControlToolbar::ToggleTrace_CanExecute() const
{
	return IsInstanceAvailable();
}

void STraceControlToolbar::ToggleTrace_Execute()
{
	if (!bIsTracing)
	{
		TraceController->WithInstance(InstanceId, [&](const FTraceStatus& Status, ITraceControllerCommands& Commands)
			{
				if (TraceTarget == ETraceTarget::Server)
				{
					Commands.Send(TraceHostAddr, TEXT(""));
				}
				else if (TraceTarget == ETraceTarget::File)
				{
					Commands.File(TEXT(""), TEXT(""));
				}
			});
		bIsTracing = true;
	}
	else
	{
		TraceController->WithInstance(InstanceId, [&](const FTraceStatus& Status, ITraceControllerCommands& Commands)
			{
				Commands.Stop();
			});
		bIsTracing = false;
	}
}

bool STraceControlToolbar::TraceSnapshot_CanExecute() const
{
	return IsInstanceAvailable();
}

void STraceControlToolbar::TraceSnapshot_Execute()
{
	TraceController->WithInstance(InstanceId, [&](const FTraceStatus& Status, ITraceControllerCommands& Commands)
	{
		if (TraceTarget == ETraceTarget::Server)
		{
			Commands.SnapshotSend(TraceHostAddr);
		}
		else if (TraceTarget == ETraceTarget::File)
		{
			Commands.SnapshotFile(TEXT(""));
		}
	});
}

bool STraceControlToolbar::TraceBookmark_CanExecute() const
{
	return IsInstanceAvailable() && bIsTracing && !bIsPaused;
}

void STraceControlToolbar::TraceBookmark_Execute()
{
	const FString BookmarkName = FDateTime::Now().ToString(TEXT("Bookmark_%Y%m%d_%H%M%S"));
	TraceController->WithInstance(InstanceId, [&](const FTraceStatus& Status, ITraceControllerCommands& Commands)
	{
		Commands.Bookmark(BookmarkName);
	});
}

bool STraceControlToolbar::TraceScreenshot_CanExecute() const
{
	return IsInstanceAvailable() && bIsTracing && !bIsPaused;
}

void STraceControlToolbar::TraceScreenshot_Execute()
{
	TraceController->WithInstance(InstanceId, [&](const FTraceStatus& Status, ITraceControllerCommands& Commands)
	{
		Commands.Screenshot(TEXT(""), false);
	});
}

bool STraceControlToolbar::ToggleStatNamedEvents_CanExecute() const
{
	return IsInstanceAvailable();
}

bool STraceControlToolbar::ToggleStatNamedEvents_IsChecked() const
{
	return bAreStatNamedEventsEnabled;
}

void STraceControlToolbar::ToggleStatNamedEvents_Execute()
{
	bAreStatNamedEventsEnabled = !bAreStatNamedEventsEnabled;
	TraceController->WithInstance(InstanceId, [&](const FTraceStatus& Status, ITraceControllerCommands& Commands)
	{
		Commands.SetStatNamedEventsEnabled(bAreStatNamedEventsEnabled);
	});
}

FText STraceControlToolbar::GetTraceTargetLabelText() const
{
	if (TraceTarget == ETraceTarget::Server)
	{
		return LOCTEXT("TraceTargetServerLabel", "Server");
	}

	return LOCTEXT("TraceTargetFileLabel", "File");
}

FText STraceControlToolbar::GetTraceTargetTooltipText() const
{
	return LOCTEXT("TraceTargetTooltip", "Set the trace target. Can only be set when trace is not running.");
}

FSlateIcon STraceControlToolbar::GetTraceTargetIcon() const
{
	if (TraceTarget == ETraceTarget::Server)
	{
		return FSlateIcon(FTraceToolsStyle::GetStyleSetName(), "TraceControl.SetTraceTargetServer");
	}

	return FSlateIcon(FTraceToolsStyle::GetStyleSetName(), "TraceControl.SetTraceTargetFile");
}

void STraceControlToolbar::OnTraceStatusUpdated(const FTraceStatus& InStatus, FTraceStatus::EUpdateType InUpdateType, ITraceControllerCommands& Commands)
{
	if (!InstanceId.IsValid() || InstanceId != InStatus.InstanceId)
	{
		return;
	}

	bIsTracing = InStatus.bIsTracing;
	bIsPaused = InStatus.bIsPaused;
	bAreStatNamedEventsEnabled = InStatus.bAreStatNamedEventsEnabled;
	bIsTracingAvailable = InStatus.TraceSystemStatus != FTraceStatus::ETraceSystemStatus::NotAvailable;

	if (InStatus.TraceSystemStatus == FTraceStatus::ETraceSystemStatus::TracingToServer)
	{
		TraceTarget = ETraceTarget::Server;
	}
	else if (InStatus.TraceSystemStatus == FTraceStatus::ETraceSystemStatus::TracingToFile)
	{
		TraceTarget = ETraceTarget::File;
	}
}

bool STraceControlToolbar::TogglePauseResume_CanExecute() const
{
	return IsInstanceAvailable() && bIsTracing;
}

FReply STraceControlToolbar::TogglePauseResume_OnClicked()
{
	if (bIsPaused)
	{
		TraceController->WithInstance(InstanceId, [&](const FTraceStatus& Status, ITraceControllerCommands& Commands)
			{
				Commands.Resume();
			});
		bIsPaused = false;
	}
	else
	{
		TraceController->WithInstance(InstanceId, [&](const FTraceStatus& Status, ITraceControllerCommands& Commands)
			{
				Commands.Pause();
			});
		bIsPaused = true;
	}

	return FReply::Handled();
}

const FSlateBrush* STraceControlToolbar::GetPauseResumeBrush() const
{
	if (bIsPaused)
	{
		return FTraceToolsStyle::GetBrush("TraceControl.ResumeTrace.Small");
	}
		
	return FTraceToolsStyle::GetBrush("TraceControl.PauseTrace.Small");
}

FText STraceControlToolbar::TogglePauseResume_GetTooltip() const
{
	if (bIsPaused)
	{
		return LOCTEXT("ResumeTraceTooltip", "Enable the channels that were enabled before trace was paused.");
	}

	return LOCTEXT("PauseTraceTooltip", "Disable all the trace channels and save the channel list so they can be enabled again with the resume command.");
}

void STraceControlToolbar::SetInstanceId(const FGuid& Id)
{
	InstanceId = Id;
	Reset();
}

void STraceControlToolbar::Reset()
{
	TraceTarget = ETraceTarget::Server;
	bIsTracing = false;
	bIsPaused = false;
	bAreStatNamedEventsEnabled = false;
	bIsTracingAvailable = false;
}

} // namespace UE::TraceTools

#undef LOCTEXT_NAMESPACE