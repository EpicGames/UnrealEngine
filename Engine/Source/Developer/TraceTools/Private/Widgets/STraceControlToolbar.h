// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ITraceController.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class ITraceController;
class FReply;
struct FSlateBrush;

namespace UE::TraceTools
{

class SToggleTraceButton;

enum ETraceTarget : uint8
{
	Server = 0,
	File = 1
};

/**
 * Implements the trace control toolbar widget.
 */
class STraceControlToolbar
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(STraceControlToolbar) { }
	SLATE_END_ARGS()

public:

	STraceControlToolbar();
	~STraceControlToolbar();

	void Construct( const FArguments& InArgs, const TSharedRef<FUICommandList>& CommandList, TSharedPtr<ITraceController> InTraceController);

	void SetInstanceId(const FGuid& Id);

private:
	void BindCommands(const TSharedRef<FUICommandList>& CommandList);

	TSharedRef<SWidget> BuildTraceTargetMenu(const TSharedRef<FUICommandList> CommandList);
	FText GetTraceTargetLabelText() const;
	FText GetTraceTargetTooltipText() const;
	FSlateIcon GetTraceTargetIcon() const;

	void OnTraceStatusUpdated(const FTraceStatus& InStatus, FTraceStatus::EUpdateType InUpdateType, ITraceControllerCommands& Commands);
	void InitializeSettings();

	bool SetTraceTarget_CanExecute() const;
	void SetTraceTarget_Execute(ETraceTarget InTraceTarget);

	bool ToggleTrace_CanExecute() const;
	void ToggleTrace_Execute();

	bool TraceSnapshot_CanExecute() const;
	void TraceSnapshot_Execute();

	bool TraceBookmark_CanExecute() const;
	void TraceBookmark_Execute();

	bool TraceScreenshot_CanExecute() const;
	void TraceScreenshot_Execute();

	bool ToggleStatNamedEvents_CanExecute() const;
	bool ToggleStatNamedEvents_IsChecked() const;
	void ToggleStatNamedEvents_Execute();

	FReply TogglePauseResume_OnClicked();
	bool TogglePauseResume_CanExecute() const;
	FText TogglePauseResume_GetTooltip() const;

	const FSlateBrush* GetPauseResumeBrush() const;

	bool IsInstanceAvailable() const;

	void Reset();

private:
	TSharedPtr<ITraceController> TraceController;

	ETraceTarget TraceTarget = ETraceTarget::Server;
	bool bIsTracing = false;
	bool bIsPaused = false;
	bool bAreStatNamedEventsEnabled = false;
	bool bIsTracingAvailable = false;
	FString TraceHostAddr;
	FGuid InstanceId;

	FDelegateHandle OnStatusReceivedDelegate;
};

} // namespace UE::TraceTools