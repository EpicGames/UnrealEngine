// Copyright Epic Games, Inc. All Rights Reserved.

#include "STraceDataFilterWidget.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Framework/Views/TreeFilterHandler.h"
#include "Misc/TextFilter.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Views/STreeView.h"

// TraceTools
#include "Models/TraceChannel.h"
#include "Models/TraceFilterPresets.h"
#include "Services/SessionTraceControllerFilterService.h"
#include "STraceObjectRowWidget.h"
#include "STraceStatistics.h"
#include "TraceToolsStyle.h"
#include "Widgets/SFilterPresetList.h"

#define LOCTEXT_NAMESPACE "STraceDataFilterWidget"

namespace UE::TraceTools
{

STraceDataFilterWidget::STraceDataFilterWidget() : bNeedsListRefresh(false), bHighlightingPreset(false)
{
}

STraceDataFilterWidget::~STraceDataFilterWidget()
{
}

void STraceDataFilterWidget::Construct(const FArguments& InArgs, TSharedPtr<ITraceController> InTraceController, TSharedPtr<ISessionTraceFilterService> InSessionFilterService)
{
	TraceController = InTraceController;
	SessionFilterService = InSessionFilterService;

	SAssignNew(ExternalScrollbar, SScrollBar)
	.AlwaysShowScrollbar(true);

	ConstructSearchBoxFilter();
	ConstructTileView();

	ChildSlot
	[		
		SNew(SBorder)
		.Padding(4)
		.BorderImage(FTraceToolsStyle::GetBrush("FilterPresets.BackgroundBorder"))
		[
			SNew( SVerticalBox )

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 2.0f))
			[
				// Filtering button and search box widgets
				SAssignNew(OptionsWidget, SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 2.0f, 0.0f)
				[
					SNew(SComboButton)
					.Visibility(EVisibility::Visible)
					.ComboButtonStyle(FTraceToolsStyle::Get(), "EventFilter.ComboButton")
					.ForegroundColor(FLinearColor::White)
					.ContentPadding(0.0f)
					.OnGetMenuContent(this, &STraceDataFilterWidget::MakeAddFilterMenu)
					.ButtonContent()
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.TextStyle(FTraceToolsStyle::Get(), "EventFilter.TextStyle")
							.Font(FTraceToolsStyle::Get().GetFontStyle("FontAwesome.9"))
							.Text(FText::FromString(FString(TEXT("\xf0b0"))) /*fa-filter*/)
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2, 0, 0, 0)
						.VAlign(VAlign_Center)
						[
							SNew( STextBlock )
							.TextStyle(FTraceToolsStyle::Get(), "EventFilter.TextStyle")
							.Text( LOCTEXT("PresetsMenuLabel", "Filter Presets") )
						]
					]
				]
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SAssignNew(SearchBoxWidget, SSearchBox)
					.SelectAllTextWhenFocused( true )
					.HintText( LOCTEXT( "SearchBoxHint", "Search Trace Events..."))
					.OnTextChanged(this, &STraceDataFilterWidget::OnSearchboxTextChanged)
				]
			]

			+ SVerticalBox::Slot()
			.Padding(0.0f, 0.0f, 0.0f, 2.0f)
			.AutoHeight()
			[
				// Presets bar widget
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SAssignNew(FilterPresetsListWidget, SFilterPresetList, SessionFilterService)
					.OnPresetChanged(this, &STraceDataFilterWidget::OnPresetChanged)
					.OnSavePreset(this, &STraceDataFilterWidget::OnSavePreset)
					.OnHighlightPreset(this, &STraceDataFilterWidget::OnHighlightPreset)
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FTraceToolsStyle::GetBrush("FilterPresets.SessionWarningBorder"))
				[
					SNew(SHorizontalBox)
					.Visibility_Lambda([this]() -> EVisibility 
					{
						return ShouldShowBanner() ? EVisibility::Visible : EVisibility::Collapsed;
					})

					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(4.0f, 0.0f, 4.0f, 0.0f)
					[
						SNew(SImage)
						.Image(FTraceToolsStyle::GetBrush("FilterPresets.WarningIcon"))
					]

					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(4.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(STextBlock)
						.Text(this, &STraceDataFilterWidget::GetWarningBannerText)
						.ColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f, 1.0f))
					]
				]
			]

			+ SVerticalBox::Slot()	
			.Padding(0.0f, 0.0f, 0.0f, 2.0f)
			.FillHeight(1.0f)
			[
				SNew(SOverlay)

				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					[
						SNew(SBorder)
						.BorderImage(FTraceToolsStyle::GetBrush("FilterPresets.TableBackground"))
						[
							TileView.ToSharedRef()
						]
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 8.0f, 0.0f, 0.0f)

					[
						SNew(SSeparator)
						.Thickness(5.0f)
						.SeparatorImage(FTraceToolsStyle::GetBrush("FilterPresets.TableBackground"))
						.Orientation(EOrientation::Orient_Horizontal)
					]

					+ SVerticalBox::Slot()
					[
						SNew(STraceStatistics, SessionFilterService)
					]
				]
			]
		]
	];

	/** Setup attribute to enable/disabled all widgets according to whether or not there is a valid session to represent */
	TAttribute<bool> EnabledAttribute;
	EnabledAttribute.Bind(this, &STraceDataFilterWidget::HasValidData);

	TileView->SetEnabled(EnabledAttribute);
	OptionsWidget->SetEnabled(EnabledAttribute);
	FilterPresetsListWidget->SetEnabled(EnabledAttribute);
}

void STraceDataFilterWidget::OnSavePreset(const TSharedPtr<ITraceFilterPreset>& Preset)
{
	if (SessionFilterService.IsValid())
	{
		/** Save to preset if one was provided, otherwise create a new one */
		if (Preset.IsValid())
		{
			Preset->Save(ListItems);
		}
		else
		{
			FFilterPresetHelpers::CreateNewPreset(ListItems);
		}
	}
}

void STraceDataFilterWidget::OnPresetChanged(const SFilterPreset& Preset)
{
	if (SessionFilterService.IsValid())
	{
		SessionFilterService->UpdateFilterPreset(Preset.GetFilterPreset(), Preset.IsEnabled());
	}
}

void STraceDataFilterWidget::OnHighlightPreset(const TSharedPtr<ITraceFilterPreset>& Preset)
{
	if (TileView.IsValid())
	{
		/** Update tileview so that any allowlisted entry (as part of Preset) is highlighted */
		TileView->ClearHighlightedItems();
		if (Preset.IsValid())
		{
			if (!bHighlightingPreset)
			{
				/** Store current expansion, so we can reset once highlighting has finished */
				bHighlightingPreset = true;
			}

			TArray<FString> Names;
			Preset->GetAllowlistedNames(Names);

			EnumerateAllItems([this, Names](TSharedPtr<ITraceObject> Object) -> void
			{
				if (Names.Contains(Object->GetName()))
				{
					TileView->SetItemHighlighted(Object, true);
				}
			});
		}
		else
		{
			bHighlightingPreset = false;
		}
	}
}

void STraceDataFilterWidget::OnSearchboxTextChanged(const FText& FilterText)
{
	bNeedsListRefresh = true;

	SearchBoxWidgetFilter->SetRawFilterText(FilterText);
}

void STraceDataFilterWidget::ConstructTileView()
{
	SAssignNew(TileView, STileView<TSharedPtr<ITraceObject>>)
	.OnGenerateTile(this, &STraceDataFilterWidget::OnGenerateRow)
	.OnContextMenuOpening(this, &STraceDataFilterWidget::OnContextMenuOpening)
	.Orientation(EOrientation::Orient_Horizontal)
	.ItemHeight(15.0f)
	.ItemWidth(150.0f)
	.ListItemsSource(&FilteredListItems);
}

void STraceDataFilterWidget::ConstructSearchBoxFilter()
{
	SearchBoxWidgetFilter = MakeShareable(new TTextFilter<TSharedPtr<ITraceObject>>(TTextFilter<TSharedPtr<ITraceObject>>::FItemToStringArray::CreateLambda([](TSharedPtr<ITraceObject> Object, TArray<FString>& OutStrings)
	{
		Object->GetSearchString(OutStrings);
	})));
}

TSharedRef<ITableRow> STraceDataFilterWidget::OnGenerateRow(TSharedPtr<ITraceObject> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/STraceDataFilterWidget"));
	return SNew(STraceObjectRowWidget, OwnerTable, InItem)
	.HighlightText(SearchBoxWidgetFilter.Get(), &TTextFilter<TSharedPtr<ITraceObject>>::GetRawFilterText);
}

TSharedRef<SWidget> STraceDataFilterWidget::MakeAddFilterMenu()
{
	return FilterPresetsListWidget->ExternalMakeFilterPresetsMenu();
}

TSharedPtr<SWidget> STraceDataFilterWidget::OnContextMenuOpening() const
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr, nullptr, /*bCloseSelfOnly=*/true);

	static const FName FilteringSectionHook("FilteringState");
	MenuBuilder.BeginSection(FilteringSectionHook, LOCTEXT("FilteringSectionLabel", "Filtering"));
	{
		MenuBuilder.AddMenuEntry(LOCTEXT("EnableAllRowsLabel", "Enable All"), LOCTEXT("EnableAllRowsTooltip", "Sets entire hierarchy to be non-filtered."), FSlateIcon(), 
			FUIAction(FExecuteAction::CreateLambda([this]()
				{
					EnumerateAllItems([this](TSharedPtr<ITraceObject> Object) -> void
					{
						Object->SetIsFiltered(false);
					});
				}),
				FCanExecuteAction(),
				FGetActionCheckState(),
				FIsActionButtonVisible::CreateLambda([this]() -> bool
				{
					return TileView->GetNumItemsSelected() == 0 && EnumerateAllItems(TFunction<bool(TSharedPtr<ITraceObject> InItem)>([this](TSharedPtr<ITraceObject> Object) -> bool
					{
						return Object->IsFiltered();
					}));
				})
			)
		);

		MenuBuilder.AddMenuEntry(LOCTEXT("DisableAllRowsLabel", "Disable All"), LOCTEXT("DisableAllRowsTooltip", "Sets entire hierarchy to be filtered."), FSlateIcon(), 
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					EnumerateAllItems([this](TSharedPtr<ITraceObject> Object) -> void
					{
						Object->SetIsFiltered(true);
					});
				}),
				FCanExecuteAction(),
				FGetActionCheckState(),
				FIsActionButtonVisible::CreateLambda([this]() -> bool
				{
					return TileView->GetNumItemsSelected() == 0 && EnumerateAllItems(TFunction<bool(TSharedPtr<ITraceObject> InItem)>([this](TSharedPtr<ITraceObject> Object) -> bool
					{
						return !Object->IsFiltered();
					}));
				})

			)
		);

		MenuBuilder.AddMenuEntry(LOCTEXT("EnableRowsLabel", "Enable Selected"), LOCTEXT("EnableRowsTooltip", "Sets the selected Node(s) to be non-filtered."), FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &STraceDataFilterWidget::EnumerateSelectedItems, TFunction<void(TSharedPtr<ITraceObject> InItem)>([this](TSharedPtr<ITraceObject> Object) -> void
				{
					Object->SetIsFiltered(false);
				})),
				FCanExecuteAction(),
				FGetActionCheckState(),
				FIsActionButtonVisible::CreateLambda([this]() -> bool
				{
					return EnumerateSelectedItems(TFunction<bool(TSharedPtr<ITraceObject> InItem)>([this](TSharedPtr<ITraceObject> Object) -> bool
					{
						return Object->IsFiltered();
					}));
				})
			)
		);

		MenuBuilder.AddMenuEntry(LOCTEXT("DisableRowsLabel", "Disable Selected"), LOCTEXT("DisableRowsTooltip", "Sets the selected Node(s) to be filtered."), FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &STraceDataFilterWidget::EnumerateSelectedItems, TFunction<void(TSharedPtr<ITraceObject> InItem)>([this](TSharedPtr<ITraceObject> Object) -> void
				{
					Object->SetIsFiltered(true);
				})),
				FCanExecuteAction(),
				FGetActionCheckState(),
				FIsActionButtonVisible::CreateLambda([this]() -> bool
				{
					return EnumerateSelectedItems(TFunction<bool(TSharedPtr<ITraceObject> InItem)>([this](TSharedPtr<ITraceObject> Object) -> bool
					{
						return !Object->IsFiltered();
					}));
				})
			)
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void STraceDataFilterWidget::SaveItemSelection()
{
	SelectedObjectNames.Empty();

	if (TileView.IsValid())
	{
		EnumerateSelectedItems([this](TSharedPtr<ITraceObject> InObject)
		{
			if (InObject.IsValid())
			{
				SelectedObjectNames.Add(InObject->GetName());
			}
		});
	}
}

void STraceDataFilterWidget::RestoreItemSelection()
{
	if (TileView.IsValid())
	{
		TArray<TSharedPtr<ITraceObject>> SelectedItems;
		EnumerateAllItems([this, &SelectedItems](TSharedPtr<ITraceObject> Object) -> void
		{
			if (SelectedObjectNames.Contains(Object->GetName()))
			{
				SelectedItems.Add(Object);
			}
		});

		TileView->SetItemSelection(SelectedItems, true);
	}
	
	SelectedObjectNames.Empty();
}

TSharedRef<ITraceObject> STraceDataFilterWidget::AddFilterableObject(const FTraceObjectInfo& Event, FString ParentName)
{
	TSharedRef<FTraceChannel> SharedItem = MakeShareable(new FTraceChannel(Event.Name, Event.Description, ParentName, Event.Id, Event.bEnabled, Event.bReadOnly, SessionFilterService));

	ListItems.Add(SharedItem);

	return SharedItem;
}

bool STraceDataFilterWidget::HasValidData() const
{
	return bHasChannelData && SessionFilterService.IsValid() && SessionFilterService->HasAvailableInstance();
}

bool STraceDataFilterWidget::ShouldShowBanner() const
{
	return !HasValidData();
}

void STraceDataFilterWidget::EnumerateSelectedItems(TFunction<void(TSharedPtr<ITraceObject> InItem)> InFunction) const
{
	TArray<TSharedPtr<ITraceObject>> SelectedObjects;
	TileView->GetSelectedItems(SelectedObjects);

	for (const TSharedPtr<ITraceObject>& Object : SelectedObjects)
	{
		InFunction(Object);
	}
}

bool STraceDataFilterWidget::EnumerateSelectedItems(TFunction<bool(TSharedPtr<ITraceObject> InItem)> InFunction) const
{
	TArray<TSharedPtr<ITraceObject>> SelectedObjects;
	TileView->GetSelectedItems(SelectedObjects);


	bool bState = false;
	for (const TSharedPtr<ITraceObject>& Object : SelectedObjects)
	{
		bState |= InFunction(Object);
	}

	return bState;
}

void STraceDataFilterWidget::EnumerateAllItems(TFunction<void(TSharedPtr<ITraceObject> InItem)> InFunction) const
{
	for (const TSharedPtr<ITraceObject>& Object : ListItems)
	{
		InFunction(Object);
	}
}

bool STraceDataFilterWidget::EnumerateAllItems(TFunction<bool(TSharedPtr<ITraceObject> InItem)> InFunction) const
{
	bool bState = false;
	for (const TSharedPtr<ITraceObject>& Object : ListItems)
	{
		bState |= InFunction(Object);
	}
	return bState;
}

void STraceDataFilterWidget::RefreshTileViewData()
{
	SyncTimeStamp = SessionFilterService->GetChannelsUpdateTimestamp();

	/** Save expansion and selection */
	SaveItemSelection();

	ListItems.Empty();

	if (SessionFilterService.IsValid())
	{
		TArray<FTraceObjectInfo> RootEvents;
		SessionFilterService->GetRootObjects(RootEvents);

		RootEvents.Sort();
		
		for (const FTraceObjectInfo& RootEvent : RootEvents)
		{
			TSharedRef<ITraceObject> TraceObject = AddFilterableObject(RootEvent, FString());
		}
	}

	RestoreItemSelection();

	FilterPresetsListWidget->RefreshPresetEnabledState();
}

void STraceDataFilterWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/STraceDataFilterWidget"));

	if (SessionFilterService.IsValid() )
	{
		if (bHasSettings == false && SessionFilterService->HasSettings())
		{
			FilterPresetsListWidget->RefreshFilterPresets();
			bHasSettings = true;
		}

		if (SessionFilterService->GetChannelsUpdateTimestamp() != SyncTimeStamp)
		{
			RefreshTileViewData();
			bNeedsListRefresh = true;
			bHasChannelData = true;
		}

		if (bNeedsListRefresh)
		{
			FilteredListItems.Empty();
			for (auto& Item : ListItems)
			{
				if (SearchBoxWidgetFilter->PassesFilter(Item))
				{
					FilteredListItems.Add(Item);
				}
			}

			TileView->RequestListRefresh();
			bNeedsListRefresh = false;
		}
	}
}

void STraceDataFilterWidget::OnSessionSelectionChanged()
{
	bHasChannelData = false;
	bHasSettings = false;
}

FText STraceDataFilterWidget::GetWarningBannerText() const
{
	if (!WarningBannerText.IsEmpty())
	{
		return WarningBannerText;
	}

	return LOCTEXT("ConnectingToSessionWarning", "Connecting to live session.");
}

} // namespace UE::TraceTools

#undef LOCTEXT_NAMESPACE // "STraceDataFilterWidget"
