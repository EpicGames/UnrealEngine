// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STileView.h"

class SHorizontalBox;
class STableViewBase;
template <typename ItemType> class STileView;
template <typename ItemType> class TTextFilter;

class ITableRow;
class ITraceController;
class SSearchBox;
class SScrollBar;

namespace UE::TraceTools
{

class SFilterPresetList;
class SSessionSelector;
struct ITraceFilterPreset;
class ITraceObject;
class ISessionTraceFilterService;
class FEventFilterService;
struct FTraceObjectInfo;

class STraceDataFilterWidget : public SCompoundWidget
{
public:
	/** Default constructor. */
	STraceDataFilterWidget();

	/** Virtual destructor. */
	virtual ~STraceDataFilterWidget();

	SLATE_BEGIN_ARGS(STraceDataFilterWidget) {}
	SLATE_END_ARGS()

	/** Constructs this widget. */
	void Construct(const FArguments& InArgs, TSharedPtr<ITraceController> InTraceController, TSharedPtr<ISessionTraceFilterService> InSessionFilterService);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void SetWarningBannerText(const FText& InText) { WarningBannerText = InText; };

protected:
	/** Callback from SFilterPresetList, should save the current tileview filter state as the specified preset */
	void OnSavePreset(const TSharedPtr<ITraceFilterPreset>& Preset);
	/** Callback from SFilterPresetList, should update filtering state according to currently active Filter Presets */
	void OnPresetChanged(const class SFilterPreset& Preset);
	/** Callback from SFilterPresetList, should highlight the tileview items encompassed by the specified preset */
	void OnHighlightPreset(const TSharedPtr<ITraceFilterPreset>& Preset);

	void OnSearchboxTextChanged(const FText& FilterText);	
	void OnItemDoubleClicked(TSharedPtr<ITraceObject> InObject) const;

	/** Constructing various widgets */
	void ConstructTileView();
	void ConstructSearchBoxFilter();

	bool HasValidData() const;
	bool ShouldShowBanner() const;
		
	void RefreshTileViewData();
		
	/** Enumerates the (currently selected) tileview items and calls InFunction for each entry */
	void EnumerateSelectedItems(TFunction<void(TSharedPtr<ITraceObject> InItem)> InFunction) const;
	void EnumerateAllItems(TFunction<void(TSharedPtr<ITraceObject> InItem)> InFunction) const;
	bool EnumerateAllItems(TFunction<bool(TSharedPtr<ITraceObject> InItem)> InFunction) const;
	bool EnumerateSelectedItems(TFunction<bool(TSharedPtr<ITraceObject> InItem)> InFunction) const;

	/** Creates and adds a new tileview object entry, given the specified data */
	TSharedRef<ITraceObject> AddFilterableObject(const FTraceObjectInfo& Event, FString ParentName);
	
	/** TileView callbacks */
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<ITraceObject> InItem, const TSharedRef<STableViewBase>& OwnerTable);
	TSharedPtr<SWidget> OnContextMenuOpening() const;

	/** Presets dropdown button callback */
	TSharedRef<SWidget> MakeAddFilterMenu();

	/** (Re) storing item selection state */
	void SaveItemSelection();
	void RestoreItemSelection();

	void OnSessionSelectionChanged();

	FText GetWarningBannerText() const;

protected:
	/** Flat list of contained items */
	TArray<TSharedPtr<ITraceObject>> ListItems;
	/** Dynamically generated array by filtering */
	TArray<TSharedPtr<ITraceObject>> FilteredListItems;

	/** Analysis filter service for the current analysis session this window is currently representing */
	TSharedPtr<ISessionTraceFilterService> SessionFilterService;
	/** Timestamp used for refreshing cached filter data */
	FDateTime SyncTimeStamp = 0;
	
	/** Wrapper for presets drop down button */
	TSharedPtr<SHorizontalBox> OptionsWidget;
	
	/** TileView widget, represents the currently connected analysis session its filtering state */
	TSharedPtr<STileView<TSharedPtr<ITraceObject>>> TileView;

	/** Flag indicating the tileview should be refreshed */
	bool bNeedsListRefresh;

	/** Search box for filtering the tileview items */
	TSharedPtr<SSearchBox> SearchBoxWidget;
	/** The text filter used by the search box */
	TSharedPtr<TTextFilter<TSharedPtr<ITraceObject>>> SearchBoxWidgetFilter;

	/** Filter presets bar widget */
	TSharedPtr<SFilterPresetList> FilterPresetsListWidget;
	
	/** Whether or not we a currently highlighting a preset */
	bool bHighlightingPreset;

	/** Cached state of which named entries were selected */
	TSet<FString> SelectedObjectNames;

	TSharedPtr<SScrollBar> ExternalScrollbar;

	TSharedPtr<ITraceController> TraceController;

	bool bHasChannelData = false;
	bool bHasSettings = false;

	FText WarningBannerText;
};

} // namespace UE::TraceTools