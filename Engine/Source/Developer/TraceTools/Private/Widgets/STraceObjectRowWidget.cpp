// Copyright Epic Games, Inc. All Rights Reserved.

#include "STraceObjectRowWidget.h"

// TraceTools
#include "Models/ITraceObject.h"
#include "TraceToolsStyle.h"

namespace UE::TraceTools
{

class FSetFilteringDragDropOp : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FSetFilteringDragDropOp, FDragDropOperation)

	virtual ~FSetFilteringDragDropOp()
	{
		for (TSharedPtr<ITraceObject> Object : Objects)
		{
			Object->SetIsFiltered(bFilterOut);
		}
	}
	
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return SNullWidget::NullWidget;
	}

	static TSharedRef<FSetFilteringDragDropOp> New(TSharedPtr<ITraceObject> Object)
	{
		TSharedRef<FSetFilteringDragDropOp> Operation = MakeShareable(new FSetFilteringDragDropOp);
		
		Operation->AddObject(Object);
		Operation->bFilterOut = !Object->IsFiltered();

		Operation->Construct();
		return Operation;
	}

	void AddObject(TSharedPtr<ITraceObject> InObject)
	{
		Objects.Add(InObject);
		InObject->SetPending();
	}

	bool WillBeFilteredOut() const { return bFilterOut; }

protected:
	bool bFilterOut;
	TSet<TSharedPtr<ITraceObject>> Objects;
};


class SFilterCheckboxWidget : public SImage
{
public:
	SLATE_BEGIN_ARGS(SFilterCheckboxWidget) {}
	SLATE_END_ARGS()

	/** Construct this widget */
	void Construct(const FArguments& InArgs, TSharedPtr<ITraceObject> InObject)
	{
		Object = InObject;
		SImage::Construct(
			SImage::FArguments()
			.Image(this, &SFilterCheckboxWidget::GetBrush)
			.ColorAndOpacity(FSlateColor::UseForeground())
		);
	}

private:

	/** Start a new drag/drop operation for this widget */
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
		{
			return FReply::Handled().BeginDragDrop(FSetFilteringDragDropOp::New(Object));
		}
		else
		{
			return FReply::Unhandled();
		}
	}

	/** If a visibility drag drop operation has entered this widget, set its actor to the new visibility state */
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
	{
		auto FilteringOp = DragDropEvent.GetOperationAs<FSetFilteringDragDropOp>();
		if (FilteringOp.IsValid())
		{
			if (Object->IsFiltered() != FilteringOp->WillBeFilteredOut() && !Object->IsPending())
			{
				FilteringOp->AddObject(Object);
			}
		}
	}

	FReply HandleClick()
	{
		if (!IsPending())
		{
			return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
		}

		return FReply::Handled();		
	}

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override
	{
		return HandleClick();
	}

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
		{
			return FReply::Unhandled();
		}

		return HandleClick();
	}

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			if (!IsPending())
			{
				ToggleIsFiltered();
			}

			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	const FSlateBrush* GetBrush() const
	{
		if (IsPending())
		{
			return IsHovered() ? FTraceToolsStyle::GetBrush("EventFilter.State.Pending_Hovered") :
				FTraceToolsStyle::GetBrush("EventFilter.State.Pending");

		}
		else
		{
			if (IsFiltered())
			{
				return IsHovered() ? FTraceToolsStyle::GetBrush("EventFilter.State.Disabled_Hovered") :
					FTraceToolsStyle::GetBrush("EventFilter.State.Disabled");
			}
			else
			{
				return IsHovered() ? FTraceToolsStyle::GetBrush("EventFilter.State.Enabled_Hovered") :
					FTraceToolsStyle::GetBrush("EventFilter.State.Enabled");
			}
		}
	}

	bool IsFiltered() const
	{
		return Object->IsFiltered();
	}

	bool IsPending() const
	{
		return Object->IsPending();
	}

	void ToggleIsFiltered()
	{
		Object->SetIsFiltered(!Object->IsFiltered());
	}
	void SetIsFiltered(bool bState)
	{
		Object->SetIsFiltered(bState);
	}

	TSharedPtr<ITraceObject> Object;
};

void STraceObjectRowWidget::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedPtr<ITraceObject> InObject)
{
	Object = InObject;
	HighlightTextAttribute = InArgs._HighlightText.Get();

	STableRow<TSharedPtr<ITraceObject>>::Construct(
		STableRow<TSharedPtr<ITraceObject>>::FArguments()
		.Padding(0.0f)
		.Content()
		[
			// Enabled checkbox
			SNew(SHorizontalBox)
			.IsEnabled(!InObject->IsReadOnly())

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 2.0f, 0.0f)
			[
				SNew(SFilterCheckboxWidget, InObject)
			]

			// Name 
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(2.0f, 0.0f, 0.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(Object->GetDisplayText())
				.ToolTipText(Object->GetTooltipText())
				.HighlightText_Lambda([this]() -> FText
				{
					// Get attribute value, populated by search box for treeview filter-ing
					FText Value = HighlightTextAttribute.Get();

					// In case it is empty allow for treeview highlighting to drive value
					if (Value.IsEmpty())
					{
						auto OwnerTable = OwnerTablePtr.Pin().ToSharedRef();
						bool bHighlight = OwnerTable->Private_IsItemHighlighted(Object);
						if (bHighlight)
						{
							Value = Object->GetDisplayText();
						}
					}

					return Value;
				})
			]
		], InOwnerTableView);
}

} // namespace UE::TraceTools