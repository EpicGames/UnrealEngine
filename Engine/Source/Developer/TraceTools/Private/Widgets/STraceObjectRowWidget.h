// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SListView.h"

class STableViewBase;

namespace UE::TraceTools
{

class ITraceObject;

class STraceObjectRowWidget : public STableRow<TSharedPtr<ITraceObject>>
{
	SLATE_BEGIN_ARGS(STraceObjectRowWidget) {}
	/** The current text to highlight */
	SLATE_ATTRIBUTE(FText, HighlightText)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedPtr<ITraceObject> InObject);
	virtual ~STraceObjectRowWidget() = default;

protected:
	/* The trace object that this tree item represents */
	TSharedPtr<ITraceObject> Object;
	TAttribute<FText> HighlightTextAttribute;

};

} // namespace UE::TraceTools
