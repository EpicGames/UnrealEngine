// Copyright Epic Games, Inc. All Rights Reserved.

#include "STraceStatistics.h"

#include "HAL/PlatformApplicationMisc.h"
#include "Internationalization/Text.h"
#include "SlateOptMacros.h"
#include "Styling/StyleColors.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

//TraceTools
#include "Services/SessionTraceControllerFilterService.h"
#include "TraceToolsStyle.h"

#define LOCTEXT_NAMESPACE "STraceStatistics"

namespace UE::TraceTools
{

STraceStatistics::STraceStatistics()
{
}

STraceStatistics::~STraceStatistics()
{
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void STraceStatistics::Construct(const FArguments& InArgs, TSharedPtr<ISessionTraceFilterService> InSessionFilterService)
{
	SessionFilterService = InSessionFilterService;

	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(EVerticalAlignment::VAlign_Top)
		[
			SNew(SBorder)
			.BorderImage(FTraceToolsStyle::GetBrush("FilterPresets.BackgroundBorder"))
			[
				SNew(SVerticalBox)
				
				+ SVerticalBox::Slot()
				.HAlign(EHorizontalAlignment::HAlign_Left)
				.Padding(0.0f, 3.0f, 0.0f, 0.0f)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
							
					+ SHorizontalBox::Slot()
					.Padding(0.0f, 2.0f, 0.0f, 0.0f)
					.AutoWidth()
					[
						SNew(STextBlock)
						.ColorAndOpacity(FSlateColor(EStyleColor::Foreground))
						.ToolTipText(LOCTEXT("TraceStatusTooltip", "The status of the tracing system."))
						.Text(LOCTEXT("TraceStatus", "Trace Status:"))
					]

					+ SHorizontalBox::Slot()
					.Padding(2.0f, 2.0f, 0.0f, 0.0f)
					.AutoWidth()
					[
						SNew(STextBlock)
						.ColorAndOpacity(FSlateColor(EStyleColor::AccentGray))
						.Text(this, &STraceStatistics::GetTraceSystemStateText)
						.ToolTipText(this, &STraceStatistics::GetTraceSystemStateTooltipText)
					]
				]

				+ SVerticalBox::Slot()
				.HAlign(EHorizontalAlignment::HAlign_Left)
				.Padding(0.0f, 3.0f, 0.0f, 0.0f)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
							
					+ SHorizontalBox::Slot()
					.Padding(0.0f, 2.0f, 0.0f, 0.0f)
					.AutoWidth()
					[
						SNew(STextBlock)
						.ColorAndOpacity(FSlateColor(EStyleColor::Foreground))
						.ToolTipText(LOCTEXT("TraceEndpointTooltip", "The endpoint the current trace is sending data to."))
						.Text(LOCTEXT("TraceEndpoint", "Trace Endpoint:"))
					]

					+ SHorizontalBox::Slot()
					.Padding(2.0f, 2.0f, 0.0f, 0.0f)
					.AutoWidth()
					[
						SNew(STextBlock)
						.ColorAndOpacity(FSlateColor(EStyleColor::AccentGray))
						.Text(this, &STraceStatistics::GetTraceEndpointText)
					]

					+ SHorizontalBox::Slot()
					.Padding(2.0f, 2.0f, 0.0f, 0.0f)
					.AutoWidth()
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.ContentPadding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Bottom)
						.ToolTipText(LOCTEXT("CopyEndpointTooltip", "Copy the value of the current endpoint."))
						.OnClicked(this, &STraceStatistics::CopyEndpoint_OnClicked)
						.Content()
						[
							SNew(SImage)
							.Image(FTraceToolsStyle::GetBrush("TraceStatistics.CopyEndpoint"))
							.Visibility(this, &STraceStatistics::GetCopyEndpointVisibility)
						]
					]
				]
				
				+ SVerticalBox::Slot()
				.Padding(0.0f, 5.0f, 0.0f, 0.0f)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SVerticalBox)

						// Trace Settings
						+ SVerticalBox::Slot()
						.HAlign(EHorizontalAlignment::HAlign_Left)
						.Padding(0.0f, 10.0f, 0.0f, 0.0f)
						.AutoHeight()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("Trace Settings", "Trace Settings"))
							.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
						]

						+ SVerticalBox::Slot()
						.HAlign(EHorizontalAlignment::HAlign_Left)
						.AutoHeight()
						[
							SNew(SHorizontalBox)

							
							+ SHorizontalBox::Slot()
							.Padding(0.0f, 2.0f, 0.0f, 0.0f)
							.AutoWidth()
							[
								SNew(STextBlock)
								.ColorAndOpacity(FSlateColor(EStyleColor::Foreground))
								.ToolTipText(LOCTEXT("ImportantEventsSettingTooltip", "The state of the Important Events cache."))
								.Text(LOCTEXT("ImportantCache", "Important Events Cache:"))
							]

							+ SHorizontalBox::Slot()
							.Padding(2.0f, 2.0f, 0.0f, 0.0f)
							.AutoWidth()
							[
								SNew(STextBlock)
								.ColorAndOpacity(FSlateColor(EStyleColor::AccentGray))
								.Text_Lambda([this]() { return this->GetSettingsOnOffText(SessionFilterService->GetSettings().bUseImportantCache); })
							]
						]

						+ SVerticalBox::Slot()
						.HAlign(EHorizontalAlignment::HAlign_Left)
						.AutoHeight()
						[
							SNew(SHorizontalBox)
							
							+ SHorizontalBox::Slot()
							.Padding(0.0f, 2.0f, 0.0f, 0.0f)
							.AutoWidth()
							[
								SNew(STextBlock)
								.ColorAndOpacity(FSlateColor(EStyleColor::Foreground))
								.ToolTipText(LOCTEXT("UseWorkerThreadTooltip", "If trace uses a worker thread. If not, TraceLog is pumped on end frame."))
								.Text(LOCTEXT("WorkerThread", "Worker Thread:"))
							]

							+ SHorizontalBox::Slot()
							.Padding(2.0f, 2.0f, 0.0f, 0.0f)
							.AutoWidth()
							[
								SNew(STextBlock)
								.ColorAndOpacity(FSlateColor(EStyleColor::AccentGray))
								.Text_Lambda([this]() { return this->GetSettingsOnOffText(SessionFilterService->GetSettings().bUseWorkerThread); })
							]
						]

						+ SVerticalBox::Slot()
						.HAlign(EHorizontalAlignment::HAlign_Left)
						.AutoHeight()
						[
							SNew(SHorizontalBox)
							
							+ SHorizontalBox::Slot()
							.Padding(0.0f, 2.0f, 0.0f, 0.0f)
							.AutoWidth()
							[
								SNew(STextBlock)
								.ColorAndOpacity(FSlateColor(EStyleColor::Foreground))
								.ToolTipText(LOCTEXT("TailSizeTooltip", "Size of the tail buffer where the last seconds of trace data are stored."))
								.Text(LOCTEXT("TailSize", "Tail Size:"))
							]

							+ SHorizontalBox::Slot()
							.Padding(2.0f, 2.0f, 0.0f, 0.0f)
							.AutoWidth()
							[
								SNew(STextBlock)
								.ColorAndOpacity(FSlateColor(EStyleColor::AccentGray))
								.Text_Lambda([this]() { return this->GetSettingsMemoryValueText(SessionFilterService->GetSettings().TailSizeBytes); })
							]
						]
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(30.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(SVerticalBox)

						// Trace statistics
						+ SVerticalBox::Slot()
						.HAlign(EHorizontalAlignment::HAlign_Left)
						.Padding(0.0f, 10.0f, 0.0f, 0.0f)
						.AutoHeight()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("Statistics", "Statistics"))
							.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
						]

						+ SVerticalBox::Slot()
						[
							SNew(SHorizontalBox)
							
							+ SHorizontalBox::Slot()
							.Padding(0.0f, 2.0, 0.0f, 0.0f)
							.AutoWidth()
							[
								SNew(STextBlock)
								.ColorAndOpacity(FSlateColor(EStyleColor::Foreground))
								.ToolTipText(LOCTEXT("BytesSentTooltip", "Number of bytes sent to server or file."))
								.Text(LOCTEXT("BytesSent", "Bytes Sent:"))
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(2.0f, 2.0, 0.0f, 0.0f)
							[
								SNew(STextBlock)
								.ColorAndOpacity(FSlateColor(EStyleColor::AccentGray))
								.Text_Lambda([this]() { return this->GetStatsMemoryValueText(SessionFilterService->GetStats().StandardStats.BytesSent); })
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(4.0f, 2.0, 0.0f, 0.0f)
							[
								SNew(STextBlock)
								.ColorAndOpacity(FSlateColor(EStyleColor::AccentGray))
								.Text_Lambda([this]() { return this->GetStatsBandwidthText(SessionFilterService->GetStats().BytesSentPerSecond); })
							]
						]

						+ SVerticalBox::Slot()
						[
							SNew(SHorizontalBox)
							
							+ SHorizontalBox::Slot()
							.Padding(0.0f, 2.0, 0.0f, 0.0f)
							.AutoWidth()
							[
								SNew(STextBlock)
								.ColorAndOpacity(FSlateColor(EStyleColor::Foreground))
								.ToolTipText(LOCTEXT("BytesTracedTooltip", "Number of (uncompressed) bytes traced from process."))
								.Text(LOCTEXT("BytesTraced", "Bytes Traced:"))
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(2.0f, 2.0, 0.0f, 0.0f)
							[
								SNew(STextBlock)
								.ColorAndOpacity(FSlateColor(EStyleColor::AccentGray))
								.Text_Lambda([this]() { return this->GetStatsMemoryValueText(SessionFilterService->GetStats().StandardStats.BytesTraced); })
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(2.0f, 2.0, 0.0f, 0.0f)
							[
								SNew(STextBlock)
								.ColorAndOpacity(FSlateColor(EStyleColor::AccentGray))
								.Text_Lambda([this]() { return this->GetStatsBandwidthText(SessionFilterService->GetStats().BytesTracedPerSecond); })
							]
						]
		
						+ SVerticalBox::Slot()
						[
							SNew(SHorizontalBox)
							
							+ SHorizontalBox::Slot()
							.Padding(0.0f, 2.0, 0.0f, 0.0f)
							.AutoWidth()
							[
								SNew(STextBlock)
								.ColorAndOpacity(FSlateColor(EStyleColor::Foreground))
								.ToolTipText(LOCTEXT("MemoryUsedTooltip", "Total memory used by TraceLog."))
								.Text(LOCTEXT("MemoryUsed", "Memory Used:"))
							]

							+ SHorizontalBox::Slot()
							.Padding(2.0f, 2.0, 0.0f, 0.0f)
							[
								SNew(STextBlock)
								.ColorAndOpacity(FSlateColor(EStyleColor::AccentGray))
								.Text_Lambda([this]() { return this->GetStatsMemoryValueText(SessionFilterService->GetStats().StandardStats.MemoryUsed); })
							]
						]

						+ SVerticalBox::Slot()
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							.Padding(0.0f, 2.0, 0.0f, 0.0f)
							.AutoWidth()
							[
								SNew(STextBlock)
								.ColorAndOpacity(FSlateColor(EStyleColor::Foreground))
								.ToolTipText(LOCTEXT("ImportantEventsMemoryTooltip", "Memory for important events."))
								.Text(LOCTEXT("ImportantEventsCache:", "Cache:"))
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(2.0f, 2.0, 0.0f, 0.0f)
							[
								SNew(STextBlock)
								.ColorAndOpacity(FSlateColor(EStyleColor::AccentGray))
								.Text_Lambda([this]() { return this->GetStatsCacheText(); })
							]
						]
					]
				]
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FText STraceStatistics::GetSettingsOnOffText(bool InValue) const
{
	if (!SessionFilterService->HasSettings())
	{
		return LOCTEXT("N/A", "N/A");
	}

	if (InValue)
	{
		return LOCTEXT("On", "On");
	}

	return LOCTEXT("Off", "Off");
};

FText STraceStatistics::GetSettingsMemoryValueText(uint64 InValue) const
{
	if (!SessionFilterService->HasSettings())
	{
		return LOCTEXT("N/A", "N/A");
	}

	FNumberFormattingOptions FormattingOptionsMem;
	FormattingOptionsMem.MaximumFractionalDigits = 2;
	FormattingOptionsMem.MinimumFractionalDigits = 2;
	FormattingOptionsMem.MinimumIntegralDigits = 1;

	return FText::AsMemory(InValue, &FormattingOptionsMem);
}

FText STraceStatistics::GetStatsMemoryValueText(uint64 InValue) const
{
	if (!SessionFilterService->HasStats())
	{
		return LOCTEXT("N/A", "N/A");
	}

	FNumberFormattingOptions FormattingOptionsMem;
	FormattingOptionsMem.MaximumFractionalDigits = 2;
	FormattingOptionsMem.MinimumFractionalDigits = 2;
	FormattingOptionsMem.MinimumIntegralDigits = 1;

	return FText::AsMemory(InValue, &FormattingOptionsMem);
}

FText STraceStatistics::GetStatsBandwidthText(uint64 InValue) const
{
	if (!SessionFilterService->HasStats())
	{
		return FText();
	}

	FNumberFormattingOptions FormattingOptionsMem;
	FormattingOptionsMem.MaximumFractionalDigits = 2;
	FormattingOptionsMem.MinimumFractionalDigits = 2;
	FormattingOptionsMem.MinimumIntegralDigits = 1;

	FText Result = FText::AsMemory(InValue, &FormattingOptionsMem);

	return FText::Format(LOCTEXT("TraceStatBandwidthFormat", "({0}/s)"), Result);
}

FText STraceStatistics::GetStatsCacheText() const
{
	if (!SessionFilterService->HasStats())
	{
		return LOCTEXT("N/A", "N/A");
	}

	FNumberFormattingOptions FormattingOptionsMem;
	FormattingOptionsMem.MaximumFractionalDigits = 2;
	FormattingOptionsMem.MinimumFractionalDigits = 2;
	FormattingOptionsMem.MinimumIntegralDigits = 1;

	const FTraceStatus::FStats& Stats = SessionFilterService->GetStats().StandardStats;
	FText CacheAllocated = FText::AsMemory(Stats.CacheAllocated, &FormattingOptionsMem);
	FText CacheUsed = FText::AsMemory(Stats.CacheUsed, &FormattingOptionsMem);
	FText CacheUnused = FText::AsMemory(Stats.CacheAllocated - Stats.CacheUsed, &FormattingOptionsMem);
	FText CacheWasted = FText::AsMemory(Stats.CacheWaste, &FormattingOptionsMem);

	return FText::Format(LOCTEXT("TraceCacheTextFormat", "{0} ({1} used + {2} unused | {3} waste)"), CacheAllocated, CacheUsed, CacheUnused, CacheWasted);
}

FText STraceStatistics::GetTraceEndpointText() const
{
	if (!SessionFilterService->HasStats() || SessionFilterService->GetTraceEndpoint().IsEmpty())
	{
		return LOCTEXT("N/A", "N/A");
	}

	return FText::FromString(SessionFilterService->GetTraceEndpoint());
}

FText STraceStatistics::GetTraceSystemStateText() const
{
	// If you update these values also check GetTraceSystemStateTooltipText.
	static_assert((uint8)FTraceStatus::ETraceSystemStatus::NotAvailable == (uint8) FTraceAuxiliary::ETraceSystemStatus::NotAvailable);
	static_assert((uint8)FTraceStatus::ETraceSystemStatus::Available == (uint8) FTraceAuxiliary::ETraceSystemStatus::Available);
	static_assert((uint8)FTraceStatus::ETraceSystemStatus::TracingToFile == (uint8) FTraceAuxiliary::ETraceSystemStatus::TracingToFile);
	static_assert((uint8)FTraceStatus::ETraceSystemStatus::TracingToServer == (uint8) FTraceAuxiliary::ETraceSystemStatus::TracingToServer);
	static_assert((uint8)FTraceStatus::ETraceSystemStatus::TracingToCustomRelay == (uint8)FTraceAuxiliary::ETraceSystemStatus::TracingToCustomRelay);
	static_assert((uint8)FTraceStatus::ETraceSystemStatus::TracingToSecureNetwork == (uint8)FTraceAuxiliary::ETraceSystemStatus::TracingToSecureNetwork);
	static_assert((uint8)FTraceStatus::ETraceSystemStatus::NumValues == (uint8) FTraceAuxiliary::ETraceSystemStatus::NumValues, "ETraceSystemStatus enum values are of out sync.");

	if (!SessionFilterService->HasStats())
	{
		return LOCTEXT("N/A", "N/A");
	}

	switch (SessionFilterService->GetTraceSystemStatus())
	{
	case FTraceStatus::ETraceSystemStatus::NotAvailable:
	{
		return LOCTEXT("TraceSystemNotAvailableText", "Not Available");
	}
	case FTraceStatus::ETraceSystemStatus::Available:
	{
		return LOCTEXT("TraceSystemAvailableText", "Available");
	}
	case FTraceStatus::ETraceSystemStatus::TracingToServer:
	{
		return LOCTEXT("TracingToServerText", "Tracing to Network");
	}
	case FTraceStatus::ETraceSystemStatus::TracingToFile:
	{
		return LOCTEXT("TracingToFileText", "Tracing to File");
	}
	case FTraceStatus::ETraceSystemStatus::TracingToCustomRelay:
	{
		return LOCTEXT("TracingToCustomRelayText", "Tracing to Custom Relay");
	}
	case FTraceStatus::ETraceSystemStatus::TracingToSecureNetwork:
	{
		return LOCTEXT("TracingToSecureNetwork", "Tracing to secure network connection");
	}
	default:
		return LOCTEXT("Unknown", "Unknown");
	}
}

FText STraceStatistics::GetTraceSystemStateTooltipText() const
{
	if (!SessionFilterService->HasStats())
	{
		return FText::GetEmpty();
	}

	switch (SessionFilterService->GetTraceSystemStatus())
	{
	case FTraceStatus::ETraceSystemStatus::NotAvailable:
	{
		return LOCTEXT("TraceSystemNotAvailableTooltipText", "Trace system is disabled at compile time. Check the UE_TRACE_ENABLED define.");
	}
	case FTraceStatus::ETraceSystemStatus::Available:
	{
		return LOCTEXT("TraceSystemAvailableTooltipText", "Trace system is available and can be started. Data might be stored in the Important Events and Tail buffers.");
	}
	case FTraceStatus::ETraceSystemStatus::TracingToServer:
	{
		return LOCTEXT("TracingToServerTooltipText", "Tracing to network (to trace server or using a direct trace connection).");
	}
	case FTraceStatus::ETraceSystemStatus::TracingToFile:
	{
		return LOCTEXT("TracingToFileTooltipText", "Tracing directly to a file.");
	}
	case FTraceStatus::ETraceSystemStatus::TracingToCustomRelay:
	{
		return LOCTEXT("TracingToCustomRelayTooltipText", "Tracing to a custom relay.");
	}
	case FTraceStatus::ETraceSystemStatus::TracingToSecureNetwork:
	{
		return LOCTEXT("TracingToSecureNetwork", "Tracing to secure network connection");
	}
	default:
		return FText::GetEmpty();
	}
}

FReply STraceStatistics::CopyEndpoint_OnClicked() const
{
	const FString& Endpoint = SessionFilterService->GetTraceEndpoint();

	if (!Endpoint.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*Endpoint);
	}

	return FReply::Handled();
}

EVisibility STraceStatistics::GetCopyEndpointVisibility() const
{
	if (!SessionFilterService->HasStats() || SessionFilterService->GetTraceEndpoint().IsEmpty())
	{
		return EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

} // namespace UE::TraceTools

#undef LOCTEXT_NAMESPACE
