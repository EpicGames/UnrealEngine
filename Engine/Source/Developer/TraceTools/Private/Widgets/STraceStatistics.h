// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ITraceController.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

struct EVisibility;
class FReply;
class ITraceController;

namespace UE::TraceTools
{

class ISessionTraceFilterService;

/**
 * A widget that displays trace settings and trace statistics
 */
class STraceStatistics
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STraceStatistics) { }
	SLATE_END_ARGS()

public:
	STraceStatistics();
	virtual ~STraceStatistics();

	void Construct( const FArguments& InArgs, TSharedPtr<ISessionTraceFilterService> SessionFilterService);

private:
	FText GetSettingsOnOffText(bool InValue) const;
	FText GetSettingsMemoryValueText(uint64 InValue) const;
	FText GetStatsMemoryValueText(uint64 InValue) const;
	FText GetStatsBandwidthText(uint64 InValue) const;
	FText GetStatsCacheText() const;
	FText GetTraceEndpointText() const;
	FText GetTraceSystemStateText() const;
	FText GetTraceSystemStateTooltipText() const;

	FReply CopyEndpoint_OnClicked() const;
	EVisibility GetCopyEndpointVisibility() const;

private:
	TSharedPtr<ISessionTraceFilterService> SessionFilterService;
};

} // namespace UE::TraceTools