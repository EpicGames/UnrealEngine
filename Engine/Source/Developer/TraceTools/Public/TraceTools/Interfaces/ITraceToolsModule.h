// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ITraceToolsModule.h: Declares the ITraceToolsModule interface.
=============================================================================*/

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class ITraceController;
class SWidget;

namespace UE::TraceTools
{

/**
 * Interface for a trace tools module.
 */
class ITraceToolsModule
	: public IModuleInterface
{
public:
	/**
	 * Creates a trace control widget that auto detects the session selected in Session Browser and can be used to control it.
	 *
	 * @return The new widget
	 */
	virtual TSharedRef<SWidget> CreateTraceControlWidget(TSharedPtr<ITraceController> InTraceController) = 0;

	/**
	 * Creates a trace control widget that controls a specific Instance Id.
	 *
	 * @return The new widget
	 */
	virtual TSharedRef<SWidget> CreateTraceControlWidget(TSharedPtr<ITraceController> InTraceController, FGuid InstanceId) = 0;

	/**
	 * Sets the InstanceId to control for the provided widget, which must be a widget returned by CreateTraceControlWidget.
	 */
	virtual void SetTraceControlWidgetInstanceId(TSharedRef<SWidget> Widget, FGuid InstanceId) = 0;

public:
	/**
	 * Virtual destructor.
	 */
	virtual ~ITraceToolsModule( ) = default;
};

} // namespace UE::TraceTools
