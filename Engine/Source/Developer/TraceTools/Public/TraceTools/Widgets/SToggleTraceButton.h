// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class ITraceController;

namespace UE::TraceTools
{

/**
 * A dynamic button that can start/stop trace. 
 * When tracing is started the button turns red and has a pulse animation.
 */
class SToggleTraceButton
	: public SCompoundWidget
{
	SLATE_DECLARE_WIDGET_API(SToggleTraceButton, SCompoundWidget, TRACETOOLS_API)

public:
	enum class EButtonSize : uint32
	{
		StatusBar = 0, // 16x16
		SlimToolbar = 1, // 20x20
	};

	DECLARE_DELEGATE(FOnToggleTraceRequested);

	SLATE_BEGIN_ARGS(SToggleTraceButton) 
		: _ButtonSize(EButtonSize::StatusBar)
		, _ShowStopButton(true)
	{ }
	
	/* An event called when the toggle trace button is clicked. */
	SLATE_EVENT(FOnToggleTraceRequested, OnToggleTraceRequested)
	
	/* Specifies if trace is running at the moment of call. */
	SLATE_ATTRIBUTE(bool, IsTraceRunning)

	/* Specifies the tooltip text that should change based on trace status. */
	SLATE_ATTRIBUTE(FText, DynamicToolTipText)

	/* Label to place in the button. */
	SLATE_ATTRIBUTE(FText, LabelText)
	
	/* Specifies the size of the button from a few presets. */
	SLATE_ARGUMENT(EButtonSize, ButtonSize)

	/* If true, the tracing icon will be replaced with a stop icon when hovered. */
	SLATE_ARGUMENT(bool, ShowStopButton)

	SLATE_END_ARGS()

public:
	TRACETOOLS_API SToggleTraceButton();
	TRACETOOLS_API virtual ~SToggleTraceButton();

	TRACETOOLS_API void Construct( const FArguments& InArgs);

private:
	FSlateColor GetRecordingButtonColor() const;
	FSlateColor GetRecordingButtonOutlineColor() const;
	FText GetRecordingButtonTooltipText() const;
	FText GetRecordingButtonLabelText() const;

	EVisibility GetStartTraceIconVisibility() const;
	EVisibility GetStopTraceIconVisibility() const;

	void ToggleTrace_OnClicked() const;

	const FSlateBrush* GetToggleTraceCenterBrush() const;
	const FSlateBrush* GetToggleTraceOutlineBrush() const;
	const FSlateBrush* GetToggleTraceStopBrush() const;

private:
	FOnToggleTraceRequested OnToggleTraceRequested;

	bool bIsTraceRecordButtonHovered = false;
	mutable double ConnectionStartTime = 0.0f;

	TSlateAttribute<bool> IsTraceRunningAttribute;
	TSlateAttribute<FText> DynamicToolTipTextAttribute;
	TSlateAttribute<FText> LabelTextAttribute;
	EButtonSize ButtonSize;
	bool bShowStopButton;
};

} // namespace UE::TraceTools