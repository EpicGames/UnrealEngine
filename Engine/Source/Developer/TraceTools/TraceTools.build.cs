// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TraceTools : ModuleRules
{
	public TraceTools(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		PrivateDependencyModuleNames.AddRange(
				new string[] {
					"ApplicationCore",
					"Core",
					"CoreUObject",
					"InputCore",
					"SessionServices",
					"Slate",
					"SlateCore",
					"Sockets",
					"TraceLog",
				}
			);

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.Add("SharedSettingsWidgets");
		}
	}
}
